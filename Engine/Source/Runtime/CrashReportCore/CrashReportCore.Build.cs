// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class CrashReportCore : ModuleRules
{
	public CrashReportCore( ReadOnlyTargetRules Target ) : base(Target)
	{
		bRequiresPlatformSDK = true;

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
                "XmlParser",
                "Analytics",
                "AnalyticsET",
				"HTTP",
                "Json",
           }
        );

		if (Target.Type != TargetType.Editor)
        {
            PrecompileForTargets = PrecompileTargetsType.None;
        }

        if (Target.Type == TargetType.Game || Target.Type == TargetType.Client || Target.Type == TargetType.Program)
        {
			PublicDependencyModuleNames.Add("CrashDebugHelper");
			PublicDefinitions.Add("WITH_CRASHREPORTER=1");
		}
		else
        {
			PublicDefinitions.Add("WITH_CRASHREPORTER=0");
		}
    }
}
