// Copyright Epic Games, Inc. All Rights Reserved.

#include "Android/AndroidErrorReport.h"
#include "../CrashReportUtil.h"
#include "CrashReportCoreConfig.h"
#include "Modules/ModuleManager.h"
#include "CrashDebugHelperModule.h"
#include "CrashDebugHelper.h"
#include "Misc/FileHelper.h"
#include "HAL/PlatformFileManager.h"

#define LOCTEXT_NAMESPACE "CrashReport"

namespace AndroidErrorReport
{
	/** Pointer to dynamically loaded crash diagnosis module */
	FCrashDebugHelperModule* CrashHelperModule;
}

void FAndroidErrorReport::Init()
{
	AndroidErrorReport::CrashHelperModule = &FModuleManager::LoadModuleChecked<FCrashDebugHelperModule>(FName("CrashDebugHelper"));
}

void FAndroidErrorReport::ShutDown()
{
	AndroidErrorReport::CrashHelperModule->ShutdownModule();
}

 FAndroidErrorReport::FAndroidErrorReport(const FString& Directory)
	: FGenericErrorReport(Directory)
{
	// Check for some specific files to merge into the report, if they exist we rename them.
	// This way if anything goes wrong during processing it will not continue to cause issues on subsequent runs.

	static const FMergeInfo MergeTypesOfInterest[] = { 
		FMergeInfo(FString("AllThreads"), FMergeInfo::EMergeType::Threads),
		FMergeInfo(FString("PlatformProperties"), FMergeInfo::EMergeType::PlatformProperties)
	};

	for (const FMergeInfo& MergeSource : MergeTypesOfInterest)
	{
		const FString MergeFilePath(ReportDirectory / MergeSource.MergeName + TEXT(".txt"));
		IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
		bool bHasFoundMergeFile = PlatformFile.FileExists(*MergeFilePath);
		if (bHasFoundMergeFile)
		{
			FString MergeTempFileName(MergeSource.MergeName +TEXT(".tmp"));
			FString MergeTempPathName = ReportDirectory / MergeTempFileName;
			PlatformFile.MoveFile(*MergeTempPathName, *MergeFilePath);
			MergeSources.Add(FMergeInfo(MergeTempPathName, MergeSource.MergeType));
			// mirror the renaming in ReportFilenames.
			ReportFilenames.Add(MergeTempFileName);
			ReportFilenames.RemoveSingle(MergeSource.MergeName + TEXT(".txt"));
		}
	}
}

FText FAndroidErrorReport::DiagnoseReport() const
{
	auto MergeIntoReport = [](const FString & MergeFilePathName, FMergeInfo::EMergeType MergeType)
	{
		// Try to load the merge source file.
		FXmlFile MergeNode(MergeFilePathName);
		if (MergeNode.IsValid())
		{
			bool bSuccess = false;
			FXmlNode* Root = MergeNode.GetRootNode();
			if(ensure(Root))
			{
				switch (MergeType)
				{
					case FMergeInfo::EMergeType::Threads:
					{
						if (Root->GetTag().Contains(TEXT("Threads")))
						{
							FPrimaryCrashProperties::Get()->Threads = Root;
							bSuccess = true;
						}
						break;
					}
					case FMergeInfo::EMergeType::PlatformProperties:
					{
						FPrimaryCrashProperties::Get()->PlatformPropertiesExtras = Root;
						bSuccess = true;
						break;
					}
				}
			}

			if( bSuccess ) // Keep unmerged data for inspection.
			{
				// delete the file as it has been added to the primary report.
				IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();
				PlatformFile.DeleteFile(*MergeFilePathName);
			}
		}
	};


	for (const FMergeInfo& MergeSource : MergeSources)
	{
		MergeIntoReport(MergeSource.MergeName, MergeSource.MergeType);
	}
	return FText();
}

#undef LOCTEXT_NAMESPACE
