// Copyright Epic Games, Inc. All Rights Reserved.

#include "CrashDescription.h"
#include "Misc/DateTime.h"
#include "HAL/PlatformProcess.h"
#include "Misc/Paths.h"
#include "Misc/Guid.h"
#include "CrashReportCoreConfig.h"

#include "CrashReportAnalytics.h"
#include "AnalyticsEventAttribute.h"
#include "IAnalyticsProviderET.h"
#include "Misc/EngineBuildSettings.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonWriter.h"

// #CrashReport: 2015-07-23 Move crashes from C:\Users\[USER]\AppData\Local\Microsoft\Windows\WER\ReportQueue to C:\Users\[USER]\AppData\Local\CrashReportClient\Saved

/*-----------------------------------------------------------------------------
	FCrashProperty
-----------------------------------------------------------------------------*/

FCrashProperty::FCrashProperty( const FString& InMainCategory, const FString& InSecondCategory, FPrimaryCrashProperties* InOwner ) 
: Owner( InOwner )
, CachedValue( TEXT("") )
, MainCategory( InMainCategory )
, SecondCategory( InSecondCategory )
, bSet( false )
{

}

FCrashProperty& FCrashProperty::operator=(const FString& NewValue)
{
	bSet = true;
	CachedValue = NewValue;
	Owner->SetCrashProperty( MainCategory, SecondCategory, CachedValue );
	return *this;
}

FCrashProperty& FCrashProperty::operator=(const TCHAR* NewValue)
{
	bSet = true;
	CachedValue = NewValue;
	Owner->SetCrashProperty( MainCategory, SecondCategory, CachedValue );
	return *this;
}


FCrashProperty& FCrashProperty::operator=(const TArray<FString>& NewValue)
{
	bSet = true;
	CachedValue = Owner->EncodeArrayStringAsXMLString( NewValue );
	Owner->SetCrashProperty( MainCategory, SecondCategory, CachedValue );
	return *this;
}


FCrashProperty& FCrashProperty::operator=(const bool NewValue)
{
	bSet = true;
	CachedValue = NewValue ? TEXT( "1" ) : TEXT( "0" );
	Owner->SetCrashProperty( MainCategory, SecondCategory, CachedValue );
	return *this;
}

FCrashProperty& FCrashProperty::operator=(const int64 NewValue)
{
	bSet = true;
	CachedValue = LexToString( NewValue );
	Owner->SetCrashProperty( MainCategory, SecondCategory, CachedValue );
	return *this;
}

const FString& FCrashProperty::AsString() const
{
	if (!bSet)
	{
		Owner->GetCrashProperty( CachedValue, MainCategory, SecondCategory );
		bSet = true;
	}
	return CachedValue;
}


bool FCrashProperty::AsBool() const
{
	return AsString().ToBool();
}

int64 FCrashProperty::AsInt64() const
{
	int64 Value = 0;
	TTypeFromString<int64>::FromString( Value, *AsString() );
	return Value;
}

FCrashPropertyXmlNode::FCrashPropertyXmlNode(const FString& InMainCategory, const FString& InSecondCategory, FPrimaryCrashProperties* InOwner)
	: Owner(InOwner)
	, MainCategory(InMainCategory)
	, SecondCategory(InSecondCategory)
	, bSet(false)
{ }

FCrashPropertyXmlNode& FCrashPropertyXmlNode::operator=(const FXmlNode* Node)
{
	bSet = true;
	Owner->SetCrashProperty(MainCategory, SecondCategory, Node);
	return *this;
}

#if WITH_CRASHREPORTER
FCrashPropertyXmlNode& FCrashPropertyXmlNode::operator=(const TArray<FCrashThreadInfo>& ThreadsInfo)
{
	if (ThreadsInfo.Num() <= 0)
	{
		return *this;
	}
	
	bSet = true;
	FXmlNode* RuntimePropertiesNode = Owner->XmlFile->GetRootNode()->FindChildNode( FGenericCrashContext::RuntimePropertiesTag );
	RuntimePropertiesNode->AppendChildNode(TEXT("Threads"));
	FXmlNode* ThreadsNode = RuntimePropertiesNode->GetChildrenNodes().Last();
	for (int32 Idx = 0; Idx < ThreadsInfo.Num(); ++Idx)
	{
		ThreadsNode->AppendChildNode(TEXT("Thread"));
		FXmlNode* ThreadNode = ThreadsNode->GetChildrenNodes().Last();
		
		const FCrashThreadInfo& ThreadInfo = ThreadsInfo[Idx];
		ThreadNode->AppendChildNode(TEXT("CallStack"), ThreadInfo.PortableCallStackString);
		ThreadNode->AppendChildNode(TEXT("IsCrashed"), ThreadInfo.IsCrashing ? TEXT("true") : TEXT("false"));
		// TODO: do we need thread register states?
		ThreadNode->AppendChildNode(TEXT("Registers"));
		ThreadNode->AppendChildNode(TEXT("ThreadID"), FString::Printf(TEXT("%d"), ThreadInfo.ThreadId));
		ThreadNode->AppendChildNode(TEXT("ThreadName"), ThreadInfo.ThreadName);
	}
	
	return *this;
}
#endif


/*-----------------------------------------------------------------------------
	FPrimaryCrashProperties
-----------------------------------------------------------------------------*/

FPrimaryCrashProperties* FPrimaryCrashProperties::Singleton = nullptr;

FPrimaryCrashProperties::FPrimaryCrashProperties()
	// At this moment only these properties can be changed by the crash report client.
	: EngineModeEx( FGenericCrashContext::RuntimePropertiesTag, TEXT("EngineModeEx"), this )
	, PlatformFullName( FGenericCrashContext::RuntimePropertiesTag, TEXT( "PlatformFullName" ), this )
	, CommandLine( FGenericCrashContext::RuntimePropertiesTag, TEXT( "CommandLine" ), this )
	, UserName( FGenericCrashContext::RuntimePropertiesTag, TEXT("UserName"), this )
	, LoginId( FGenericCrashContext::RuntimePropertiesTag, TEXT( "LoginId" ), this )
	, EpicAccountId( FGenericCrashContext::RuntimePropertiesTag, TEXT( "EpicAccountId" ), this )
	, GameSessionID( FGenericCrashContext::RuntimePropertiesTag, TEXT( "GameSessionID" ), this )
	// Multiline properties
	, CallStack( FGenericCrashContext::RuntimePropertiesTag, TEXT( "CallStack" ), this )
	, PCallStack( FGenericCrashContext::RuntimePropertiesTag, TEXT( "PCallStack" ), this )
	, PCallStackHashProperty( FGenericCrashContext::RuntimePropertiesTag, TEXT( "PCallStackHash" ), this )
	, SourceContext( FGenericCrashContext::RuntimePropertiesTag, TEXT( "SourceContext" ), this )
	, Modules( FGenericCrashContext::RuntimePropertiesTag, TEXT( "Modules" ), this )
	, UserDescription( FGenericCrashContext::RuntimePropertiesTag, TEXT( "UserDescription" ), this )
	, UserActivityHint( FGenericCrashContext::RuntimePropertiesTag, TEXT( "UserActivityHint" ), this )
	, ErrorMessage( FGenericCrashContext::RuntimePropertiesTag, TEXT( "ErrorMessage" ), this )
	, FullCrashDumpLocation( FGenericCrashContext::RuntimePropertiesTag, TEXT( "FullCrashDumpLocation" ), this )
	, TimeOfCrash( FGenericCrashContext::RuntimePropertiesTag, TEXT( "TimeOfCrash" ), this )
	, bAllowToBeContacted( FGenericCrashContext::RuntimePropertiesTag, TEXT( "bAllowToBeContacted" ), this )
	, CrashReporterMessage( FGenericCrashContext::RuntimePropertiesTag, TEXT( "CrashReporterMessage" ), this )
	, PlatformCallbackResult(FGenericCrashContext::PlatformPropertiesTag, TEXT("PlatformCallbackResult"), this)
	, CrashReportClientVersion(FGenericCrashContext::RuntimePropertiesTag, TEXT("CrashReportClientVersion"), this)
	, CPUBrand(FGenericCrashContext::RuntimePropertiesTag, TEXT("CPUBrand"), this)
	, Threads(FGenericCrashContext::RuntimePropertiesTag, TEXT("Threads"), this)
	, PlatformPropertiesExtras(FGenericCrashContext::PlatformPropertiesTag, TEXT(""), this)
	, bIsOOM(false)
	, bLowMemoryWarning(false)
	, bInBackground(false)
	, bIsRequestingExit(false)
	, XmlFile( nullptr )
{
	CrashVersion = ECrashDescVersions::VER_1_NewCrashFormat;
	CrashDumpMode = ECrashDumpMode::Default;
	bHasMiniDumpFile = false;
	bHasLogFile = false;
	bHasPrimaryData = false;
	NumMinidumpFramesToIgnore = 0;
	CrashSignal = 0;
}

void FPrimaryCrashProperties::Shutdown()
{
	delete Get();
}

void FPrimaryCrashProperties::UpdateIDs()
{
	// clear EpicAccountId if user doesn't want to be contacted
	const bool bAddPersonalData = FCrashReportCoreConfig::Get().GetAllowToBeContacted();
	bAllowToBeContacted = bAddPersonalData;
	if (!bAddPersonalData)
	{
		EpicAccountId = FString();
	}

	// Only send user name for internal builds
	if (FEngineBuildSettings::IsInternalBuild())
	{
		// Remove periods from user names to match AutoReporter user names
		// The name prefix is read by CrashRepository.AddNewCrash in the website code
		UserName = FString( FPlatformProcess::UserName() ).Replace( TEXT( "." ), TEXT( "" ) );
	}
	else
	{
		UserName = FString();
	}

	LoginId = FPlatformMisc::GetLoginId();
}

void FPrimaryCrashProperties::ReadXML( const FString& CrashContextFilepath, const TCHAR* Buffer  )
{
	XmlFilepath = CrashContextFilepath;

	XmlFile = new FXmlFile();

	// Special hack to work around some whitespace issues in multi-line element-content blocks like callstacks
	XmlFile->EnableAttemptToPreserveWhitespaceHack();

	if (Buffer)
	{
		XmlFile->LoadFile(Buffer, EConstructMethod::ConstructFromBuffer);
	}
	else
	{
		XmlFile->LoadFile(XmlFilepath);
	}
}

void FPrimaryCrashProperties::UpdateDefaults()
{
	if (XmlFile->IsValid())
	{
		TimeOfCrash = FDateTime::UtcNow().GetTicks();
		UpdateIDs();
	}
}

void FPrimaryCrashProperties::SetCrashGUID( const FString& Filepath )
{
	FString CrashDirectory = FPaths::GetPath( Filepath );
	FPaths::NormalizeDirectoryName( CrashDirectory );
	// Grab the last component...
	CrashGUID = FPaths::GetCleanFilename( CrashDirectory );
}

FString FPrimaryCrashProperties::EncodeArrayStringAsXMLString( const TArray<FString>& ArrayString ) const
{
	const FString Encoded = FString::Join( ArrayString, TEXT("\n") );
	return Encoded;
}

/**
 * @EventName CrashReportClient.ReportEnsure
 *
 * @Trigger Sends just before the CrashReportClient attempts to upload an ensure (a non-fatal error NOT a crash) report
 *
 * @Type Client
 * @Owner Wes.Hunt
 *
 * @EventParam bHasPrimaryData - Whether the crash loaded data successfully from a crash context or legacy metadata file that was saved by the crashed process ("true" or "false")
 * @EventParam CrashVersion" - Describes the version of the crash data pipeline we used on the client side (1 = Legacy metadata based, 2 = unused, 3 = New crash context based)
 * @EventParam CrashGUID - GUID for this event generated by the crashed process (e.g. Windows format is UECC-Windows-1F07494140C2669B52AC96A1C4D20F83_0000 with the last four digits representing the index of event within a single session)
 * @EventParam PlatformCallbackResult - Platform-specific UE Core value. (integer)
 * @EventParam GameName - The name of the game that crashed from FApp::GetProjectName(). (e.g. UE-QAGame)
 * @EventParam EngineVersion - The engine version of the crashed process from FEngineVersion::Current(). (e.g. 4.13.1-3142249+++UE+Release-4.13)
 * @EventParam CrashReportClientVersion - CrashReportClient version, set in its config. (e.g. "1.0.0")
 * @EventParam LoginID - Unique string associated with the login account of the current machine from FPlatformMisc::GetLoginId().
 * @EventParam UserName - Username of the user of the process that crashed from FPlatformProcess::UserName().
 * @EventParam EpicAccountId - Epic account ID for the user who last used the Launcher on this machine.
 * @EventParam Platform - Detailed platform name string. (e.g. "Win64 [Windows 7 Service Pack 1 64b]")
 * @EventParam TimeOfCrash - Time in UTC that the crash was read by the Crash Report Client in raw ticks (e.g. "636153351834530000", ticks are 100-nanosecond intervals that have elapsed since 12:00:00 midnight, January 1, 0001)
 * @EventParam EngineMode - The type of process that crashed ("Commandlet", "Game", "Editor" or "Server")
 * @EventParam EngineModeEx - The "vanilla" status of the engine, whether it's a totally standard, unmodified Epic-distributed build. ("Unset", "Vanilla" or "Dirty")
 * @EventParam AppDefaultLocale - The ICU default locale string or "en" if ICU is not enabled.
 * @EventParam UserActivityHint - Application-specific user activity string, if set in the crashed process. The meaning is game/app-specific.
 * @EventParam GameSessionID - Application-specific session Id, if set in the crashed process.
 * @EventParam PCallStackHash - The hash of the portable callstack
 * @EventParam CrashSignal - The signal that was raised to enter the crash handler
 * @EventParam DeploymentName - Deployment name, also known as EpicApp. (e.g. "DevPlaytest", "PublicTest", "Live", etc)
 * @EventParam EngineData - Json object containing any context that had been added to the crash context using FGenericCrashContext::SetEngineData.
 * @EventParam GameData - Json object containing any context that had been added to the crash context using FGenericCrashContext::SetGameData.
 */
void SendPreUploadEnsureAnalytics(const TArray<FAnalyticsEventAttribute>& InCrashAttributes)
{
	FCrashReportAnalytics::GetProvider().RecordEvent(TEXT("CrashReportClient.ReportEnsure"), InCrashAttributes);
}

/**
 * @EventName CrashReportClient.ReportCrash
 *
 * @Trigger Sends just before the CrashReportClient attempts to upload a crash report
 *
 * @Type Client
 * @Owner Wes.Hunt
 *
 * @EventParam bHasPrimaryData - Whether the crash loaded data successfully from a crash context or legacy metadata file that was saved by the crashed process ("true" or "false")
 * @EventParam CrashVersion" - Describes the version of the crash data pipeline we used on the client side (1 = Legacy metadata based, 2 = unused, 3 = New crash context based)
 * @EventParam CrashGUID - GUID for this event generated by the crashed process (e.g. Windows format is UECC-Windows-1F07494140C2669B52AC96A1C4D20F83_0000 with the last four digits representing the index of event within a single session)
 * @EventParam PlatformCallbackResult - Platform-specific UE Core value. (integer)
 * @EventParam GameName - The name of the game that crashed from FApp::GetProjectName(). (e.g. UE-QAGame)
 * @EventParam EngineVersion - The engine version of the crashed process from FEngineVersion::Current(). (e.g. 4.13.1-3142249+++UE+Release-4.13)
 * @EventParam CrashReportClientVersion - CrashReportClient version, set in its config. (e.g. "1.0.0")
 * @EventParam LoginID - Unique string associated with the login account of the current machine from FPlatformMisc::GetLoginId().
 * @EventParam UserName - Username of the user of the process that crashed from FPlatformProcess::UserName().
 * @EventParam EpicAccountId - Epic account ID for the user who last used the Launcher on this machine.
 * @EventParam Platform - Detailed platform name string. (e.g. "Win64 [Windows 7 Service Pack 1 64b]")
 * @EventParam TimeOfCrash - Time in UTC that the crash was read by the Crash Report Client in raw ticks (e.g. "636153351834530000", ticks are 100-nanosecond intervals that have elapsed since 12:00:00 midnight, January 1, 0001)
 * @EventParam EngineMode - The type of process that crashed ("Commandlet", "Game", "Editor" or "Server")
 * @EventParam EngineModeEx - The "vanilla" status of the engine, whether it's a totally standard, unmodified Epic-distributed build. ("Unset", "Vanilla" or "Dirty")
 * @EventParam AppDefaultLocale - The ICU default locale string or "en" if ICU is not enabled.
 * @EventParam UserActivityHint - Application-specific user activity string, if set in the crashed process. The meaning is game/app-specific.
 * @EventParam GameSessionID - Application-specific session Id, if set in the crashed process.
 * @EventParam PCallStackHash - The hash of the portable callstack
 * @EventParam CrashSignal - The signal that was raised to enter the crash handler
 * @EventParam DeploymentName - Deployment name, also known as EpicApp. (e.g. "DevPlaytest", "PublicTest", "Live", etc)
 * @EventParam EngineData - Json object containing any context that had been added to the crash context using FGenericCrashContext::SetEngineData.
 * @EventParam GameData - Json object containing any context that had been added to the crash context using FGenericCrashContext::SetGameData.
 */
void SendPreUploadCrashAnalytics(const TArray<FAnalyticsEventAttribute>& InCrashAttributes)
{
	FCrashReportAnalytics::GetProvider().RecordEvent(TEXT("CrashReportClient.ReportCrash"), InCrashAttributes);
}

void FPrimaryCrashProperties::SendPreUploadAnalytics()
{
	TArray<FAnalyticsEventAttribute> CrashAttributes;
	MakeCrashEventAttributes(CrashAttributes);

	if (FCrashReportAnalytics::IsAvailable())
	{
		if (bIsEnsure)
		{
			SendPreUploadEnsureAnalytics(CrashAttributes);
		}
		else
		{
			SendPreUploadCrashAnalytics(CrashAttributes);
		}
	}
}

/**
  * @EventName CrashReportClient.ReportEnsureUploaded
  *
  * @Trigger Sends after the CrashReportClient successfully uploads an ensure (a non-fatal error NOT a crash) report.
  *
  * @Type Client
  * @Owner Wes.Hunt
  *
  * @EventParam Result - If the HTTP end point could successfully be reached
  * @EventParam ResponseCode - HTTP Response code from the end point.
  * @EventParam Duration - Time to compress and send request.
  * @EventParam PayloadSize - Compressed size of payload (attached files)
  * @EventParam ReportCount - Number of reports uploaded (if more than one above values are accumulative or last observed)
  *
  */
void SendPostUploadEnsureAnalytics(const TArray<FAnalyticsEventAttribute>& InCrashAttributes)
{
	FCrashReportAnalytics::GetProvider().RecordEvent(TEXT("CrashReportClient.ReportEnsureUploaded"), InCrashAttributes);
}

/**
 * @EventName CrashReportClient.ReportCrashUploaded
 *
 * @Trigger Sends after the CrashReportClient successfully uploads a crash report.
 *
 * @Type Client
 * @Owner Wes.Hunt
 *
 * @EventParam Result - If the HTTP end point could successfully be reached
 * @EventParam ResponseCode - HTTP Response code from the end point.
 * @EventParam Duration - Time to compress and send request.
 * @EventParam PayloadSize - Compressed size of payload (attached files)
 *
 */
void SendPostUploadCrashAnalytics(const TArray<FAnalyticsEventAttribute>& InCrashAttributes)
{
	FCrashReportAnalytics::GetProvider().RecordEvent(TEXT("CrashReportClient.ReportCrashUploaded"), InCrashAttributes);
}

void FPrimaryCrashProperties::SendPostUploadAnalytics(double Duration, bool bResult, int32 ResponseCode, uint32 PayloadSize, uint32 ReportCount)
{
	if (FCrashReportAnalytics::IsAvailable())
	{
		const TArray<FAnalyticsEventAttribute> CrashAttributes = {
			FAnalyticsEventAttribute(TEXT("Duration"), Duration),
			FAnalyticsEventAttribute(TEXT("Result"), bResult),
			FAnalyticsEventAttribute(TEXT("ResponseCode"), ResponseCode),
			FAnalyticsEventAttribute(TEXT("PayloadSize"), PayloadSize),
			FAnalyticsEventAttribute(TEXT("ReportCount"), ReportCount),
		};
		
		if (bIsEnsure)
		{
			SendPostUploadEnsureAnalytics(CrashAttributes);
		}
		else
		{
			SendPostUploadCrashAnalytics(CrashAttributes);
		}
	}
}

namespace CrashDescriptionInternal
{
	// Json writer subclass to allow us to avoid using a SharedPtr to write basic Json.
	typedef TCondensedJsonPrintPolicy<TCHAR> FPrintPolicy;
	class FAnalyticsJsonWriter : public TJsonStringWriter<FPrintPolicy>
	{
	public:
		explicit FAnalyticsJsonWriter(FString* Out) : TJsonStringWriter<FPrintPolicy>(Out, 0)
		{
		}
	};
}

void FPrimaryCrashProperties::MakeCrashEventAttributes(TArray<FAnalyticsEventAttribute>& OutCrashAttributes) const
{
	OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("bHasPrimaryData"), bHasPrimaryData));
	OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("CrashVersion"), (int32)CrashVersion));
	OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("CrashGUID"), CrashGUID));
	OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("PlatformCallbackResult"), PlatformCallbackResult.AsString()));
	OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("CrashType"), CrashType));

	//	AppID = GameName
	OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("GameName"), GameName));

	//	AppVersion = EngineVersion
	OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("EngineVersion"), EngineVersion.ToString()));

	OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("CrashReportClientVersion"), CrashReportClientVersion.AsString()));

	// @see UpdateIDs()
	OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("LoginID"), LoginId.AsString()));
	OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("UserName"), UserName.AsString()));
	OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("EpicAccountId"), EpicAccountId.AsString()));

	OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("Platform"), PlatformFullName.AsString()));
	OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("TimeOfCrash"), TimeOfCrash.AsString()));
	OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("EngineMode"), EngineMode));
	OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("EngineModeEx"), EngineModeEx.AsString()));
	OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("AppDefaultLocale"), AppDefaultLocale));

	OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("UserActivityHint"), UserActivityHint.AsString()));
	OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("GameSessionID"), GameSessionID.AsString()));
	OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("DeploymentName"), DeploymentName));
	OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("PCallStackHash"), PCallStackHash));
	OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("CrashSignal"), CrashSignal));
	OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("CPUBrand"), CPUBrand.AsString()));
	OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("bIsOOM"), bIsOOM ? TEXT("true") : TEXT("false")));
	OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("bLowMemoryWarning"), bLowMemoryWarning ? TEXT("true") : TEXT("false")));
	OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("bInBackground"), bInBackground ? TEXT("true") : TEXT("false")));
	OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("bIsRequestingExit"), bIsRequestingExit ? TEXT("true") : TEXT("false")));

	if (XmlFile->IsValid())
	{
		// Add arbitrary platform specific data
		const FXmlNode* PlatformPropertiesNode = XmlFile->GetRootNode()->FindChildNode( FGenericCrashContext::PlatformPropertiesTag );
		if (PlatformPropertiesNode)
		{
			FString PlatformPropertiesAttributes;
			CrashDescriptionInternal::FAnalyticsJsonWriter JsonWriter(&PlatformPropertiesAttributes);
			JsonWriter.WriteObjectStart();
			for (const FXmlNode* ChildNode : PlatformPropertiesNode->GetChildrenNodes())
			{
				JsonWriter.WriteValue(ChildNode->GetTag(), ChildNode->GetContent());
			}
			JsonWriter.WriteObjectEnd();
			JsonWriter.Close();
			OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("PlatformProperties"), FJsonFragment(MoveTemp(PlatformPropertiesAttributes))));
		}
		
		// Add arbitrary engine data
		const FXmlNode* EngineNode = XmlFile->GetRootNode()->FindChildNode( FGenericCrashContext::EngineDataTag );
		if (EngineNode)
		{
			FString EngineDataAttributes;
			CrashDescriptionInternal::FAnalyticsJsonWriter JsonWriter(&EngineDataAttributes);
			JsonWriter.WriteObjectStart();
			for (const FXmlNode* ChildNode : EngineNode->GetChildrenNodes())
			{
				JsonWriter.WriteValue(ChildNode->GetTag(), ChildNode->GetContent());
			}
			JsonWriter.WriteObjectEnd();
			JsonWriter.Close();
			OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("EngineData"), FJsonFragment(MoveTemp(EngineDataAttributes))));
		}

		// Add arbitrary game data
		const FXmlNode* GameNode = XmlFile->GetRootNode()->FindChildNode( FGenericCrashContext::GameDataTag );
		if (GameNode)
		{
			FString GameDataAttributes;
			CrashDescriptionInternal::FAnalyticsJsonWriter JsonWriter(&GameDataAttributes);
			JsonWriter.WriteObjectStart();
			for (const FXmlNode* ChildNode : GameNode->GetChildrenNodes())
			{
				JsonWriter.WriteValue(ChildNode->GetTag(), ChildNode->GetContent());
			}
			JsonWriter.WriteObjectEnd();
			JsonWriter.Close();
			OutCrashAttributes.Add(FAnalyticsEventAttribute(TEXT("GameData"), FJsonFragment(MoveTemp(GameDataAttributes))));
		}
	}
	else
	{
		FString KeyName = FString("EngineData.InvalidXML");
		OutCrashAttributes.Add(FAnalyticsEventAttribute(*KeyName, true));
	}
}

void FPrimaryCrashProperties::Save()
{
	if (XmlFile->IsValid())
	{
		XmlFile->Save( XmlFilepath );
	}
}

/*-----------------------------------------------------------------------------
	FCrashContextReader
-----------------------------------------------------------------------------*/

FCrashContext::FCrashContext(const FString& CrashContextFilepath, const TCHAR* Buffer, bool bIsReadOnly)
{
	ReadXML(CrashContextFilepath, Buffer);
	if (!bIsReadOnly)
	{
		UpdateDefaults();
	}
	SetupPrimaryCrashProperties();
}

void FCrashContext::SetupPrimaryCrashProperties()
{
	const bool bIsValid = XmlFile->IsValid();
	if (bIsValid)
	{
		RestartCommandLine = CommandLine.AsString();

		// Setup properties required for the analytics.
		GetCrashProperty(CrashVersion, FGenericCrashContext::RuntimePropertiesTag, TEXT("CrashVersion"));
		GetCrashProperty(CrashGUID, FGenericCrashContext::RuntimePropertiesTag, TEXT("CrashGUID"));
		GetCrashProperty(CrashDumpMode, FGenericCrashContext::RuntimePropertiesTag, TEXT("CrashDumpMode"));
		GetCrashProperty(GameName, FGenericCrashContext::RuntimePropertiesTag, TEXT("GameName"));
		GetCrashProperty(ExecutableName, FGenericCrashContext::RuntimePropertiesTag, TEXT("ExecutableName"));
		GetCrashProperty(EngineVersion, FGenericCrashContext::RuntimePropertiesTag, TEXT("EngineVersion"));

		GetCrashProperty( BaseDir, FGenericCrashContext::RuntimePropertiesTag, TEXT( "BaseDir" ) );
		FString Misc_OSVersionMajor;
		GetCrashProperty( Misc_OSVersionMajor, FGenericCrashContext::RuntimePropertiesTag, TEXT( "Misc.OSVersionMajor" ) );
		FString Misc_OSVersionMinor;
		GetCrashProperty( Misc_OSVersionMinor, FGenericCrashContext::RuntimePropertiesTag, TEXT( "Misc.OSVersionMinor" ) );

		bool Misc_Is64bitOperatingSystem = false;
		GetCrashProperty( Misc_Is64bitOperatingSystem, FGenericCrashContext::RuntimePropertiesTag, TEXT( "Misc.Is64bitOperatingSystem" ) );

		// Extract the Platform component.
		TArray<FString> SubDirs;
		BaseDir.ParseIntoArray( SubDirs, TEXT( "/" ), true );
		const int SubDirsNum = SubDirs.Num();
		const FString PlatformName = SubDirsNum > 0 ? SubDirs[SubDirsNum - 1] : TEXT( "" );
		if (Misc_OSVersionMajor.Len() > 0)
		{
			PlatformFullName = FString::Printf( TEXT( "%s [%s %s %s]" ), *PlatformName, *Misc_OSVersionMajor, *Misc_OSVersionMinor, Misc_Is64bitOperatingSystem ? TEXT( "64b" ) : TEXT( "32b" ) );
		}
		else
		{
			PlatformFullName = PlatformName;
		}

		GetCrashProperty( EngineMode, FGenericCrashContext::RuntimePropertiesTag, TEXT( "EngineMode" ) );
		GetCrashProperty( DeploymentName, FGenericCrashContext::RuntimePropertiesTag, TEXT( "DeploymentName" ) );
		GetCrashProperty( AppDefaultLocale, FGenericCrashContext::RuntimePropertiesTag, TEXT( "AppDefaultLocale" ) );
		GetCrashProperty( bIsEnsure, FGenericCrashContext::RuntimePropertiesTag, TEXT("IsEnsure"));
		GetCrashProperty( CrashType, FGenericCrashContext::RuntimePropertiesTag, TEXT("CrashType"));
		GetCrashProperty( NumMinidumpFramesToIgnore, FGenericCrashContext::RuntimePropertiesTag, TEXT("NumMinidumpFramesToIgnore"));
		GetCrashProperty( PCallStackHash, FGenericCrashContext::RuntimePropertiesTag, TEXT("PCallStackHash"));
		GetCrashProperty( CrashSignal, FGenericCrashContext::PlatformPropertiesTag, TEXT("CrashSignal"));
		GetCrashProperty( bIsOOM, FGenericCrashContext::RuntimePropertiesTag, TEXT("MemoryStats.bIsOOM"));
		GetCrashProperty( bLowMemoryWarning, FGenericCrashContext::GameDataTag, TEXT("bLowMemoryCalled"));
		GetCrashProperty( bInBackground, FGenericCrashContext::GameDataTag, TEXT("bInBackground"));
		GetCrashProperty( bIsRequestingExit, FGenericCrashContext::RuntimePropertiesTag, TEXT("IsRequestingExit"));
		GetCrashProperty( CrashScreenshotWidth, FGenericCrashContext::EngineDataTag, TEXT("CrashScreenshotWidth"));
		GetCrashProperty( CrashScreenshotHeight, FGenericCrashContext::EngineDataTag, TEXT("CrashScreenshotHeight"));

		FString CPUBrandString;
		GetCrashProperty( CPUBrandString, FGenericCrashContext::RuntimePropertiesTag, TEXT("Misc.CPUBrand"));
		CPUBrand = CPUBrandString;

		if (CrashDumpMode == ECrashDumpMode::FullDump)
		{
			// Set the full dump crash location when we have a full dump.
			const FString LocationForBranch = FCrashReportCoreConfig::Get().GetFullCrashDumpLocationForBranch( EngineVersion.GetBranch() );
			if (!LocationForBranch.IsEmpty())
			{
				FullCrashDumpLocation = LocationForBranch / CrashGUID + TEXT("_") + EngineVersion.ToString();
			}
		}

		bHasPrimaryData = true;
	}
}

/*-----------------------------------------------------------------------------
	FCrashDescription
-----------------------------------------------------------------------------*/

FCrashWERContext::FCrashWERContext( const FString& WERXMLFilepath, bool bIsReadOnly )
	: FPrimaryCrashProperties()
{
	ReadXML( WERXMLFilepath );
	if (!bIsReadOnly)
	{
		UpdateDefaults();
	}
	CrashGUID = FPaths::GetCleanFilename( FPaths::GetPath( WERXMLFilepath ) );

	const bool bIsValid = XmlFile->IsValid();
	if (bIsValid)
	{
		FString BuildVersion;
		FString BranchName;
		uint32 BuiltFromCL = 0;
		int EngineVersionComponents = 0;

		GetCrashProperty( GameName, TEXT( "ProblemSignatures" ), TEXT( "Parameter0" ) );

		GetCrashProperty( BuildVersion, TEXT( "ProblemSignatures" ), TEXT( "Parameter1" ) );
		if (!BuildVersion.IsEmpty())
		{
			EngineVersionComponents++;
		}

		FString Parameter8Value;
		GetCrashProperty( Parameter8Value, TEXT( "ProblemSignatures" ), TEXT( "Parameter8" ) );
		if (!Parameter8Value.IsEmpty())
		{
			TArray<FString> ParsedParameters8;
			Parameter8Value.ParseIntoArray( ParsedParameters8, TEXT( "!" ), false );

			if (ParsedParameters8.Num() > 1)
			{
				CommandLine = FGenericCrashContext::UnescapeXMLString( ParsedParameters8[1] );
				CrashDumpMode = CommandLine.AsString().Contains( TEXT( "-fullcrashdump" ) ) ? ECrashDumpMode::FullDump : ECrashDumpMode::Default;
			}

			if (ParsedParameters8.Num() > 2)
			{
				ErrorMessage = ParsedParameters8[2];
			}
		}

		RestartCommandLine = CommandLine.AsString();

		FString Parameter9Value;
		GetCrashProperty( Parameter9Value, TEXT( "ProblemSignatures" ), TEXT( "Parameter9" ) );
		if (!Parameter9Value.IsEmpty())
		{
			TArray<FString> ParsedParameters9;
			Parameter9Value.ParseIntoArray( ParsedParameters9, TEXT( "!" ), false );

			if (ParsedParameters9.Num() > 0)
			{
				BranchName = ParsedParameters9[0].Replace( TEXT( "+" ), TEXT( "/" ), ESearchCase::CaseSensitive);

				const FString DepotRoot = "//depot/";
				if (BranchName.StartsWith( DepotRoot ))
				{
					BranchName.MidInline( DepotRoot.Len(), MAX_int32, EAllowShrinking::No );
				}
				EngineVersionComponents++;
			}

			if (ParsedParameters9.Num() > 1)
			{
				const FString BaseDirectory = ParsedParameters9[1];

				TArray<FString> SubDirs;
				BaseDirectory.ParseIntoArray( SubDirs, TEXT( "/" ), true );
				const int SubDirsNum = SubDirs.Num();
				const FString PlatformName = SubDirsNum > 0 ? SubDirs[SubDirsNum - 1] : TEXT( "" );

				FString Product;
				GetCrashProperty( Product, TEXT( "OSVersionInformation" ), TEXT( "Product" ) );
				if (Product.Len() > 0)
				{
					PlatformFullName = FString::Printf( TEXT( "%s [%s]" ), *PlatformName, *Product );
				}
				else
				{
					PlatformFullName = PlatformName;
				}
			}

			if (ParsedParameters9.Num() > 2)
			{
				EngineMode = ParsedParameters9[2];
			}

			if (ParsedParameters9.Num() > 3)
			{
				TTypeFromString<uint32>::FromString( BuiltFromCL, *ParsedParameters9[3] );
				EngineVersionComponents++;
			}
		}

		// We have all three components of the engine version, so initialize it.
		if (EngineVersionComponents == 3)
		{
			InitializeEngineVersion( BuildVersion, BranchName, BuiltFromCL );
		}

		GetCrashProperty(DeploymentName, TEXT("DynamicSignatures"), TEXT("DeploymentName"));
		GetCrashProperty(bIsEnsure, TEXT("DynamicSignatures"), TEXT("IsEnsure"));

		FString EngineModeExString;
		GetCrashProperty(EngineModeExString, TEXT("DynamicSignatures"), TEXT("EngineModeEx"));
		EngineModeEx = EngineModeExString;

		bHasPrimaryData = true;
	}
}

void FCrashWERContext::InitializeEngineVersion( const FString& BuildVersion, const FString& BranchName, uint32 BuiltFromCL )
{
	uint16 Major = 0;
	uint16 Minor = 0;
	uint16 Patch = 0;
 
	TArray<FString> ParsedBuildVersion;
	BuildVersion.ParseIntoArray( ParsedBuildVersion, TEXT( "." ), false );
 
	if (ParsedBuildVersion.Num() >= 3)
	{
		TTypeFromString<uint16>::FromString( Patch, *ParsedBuildVersion[2] );
	}
 
	if (ParsedBuildVersion.Num() >= 2)
	{
		TTypeFromString<uint16>::FromString( Minor, *ParsedBuildVersion[1] );
	}
 
	if (ParsedBuildVersion.Num() >= 1)
	{
		TTypeFromString<uint16>::FromString( Major, *ParsedBuildVersion[0] );
	}
 
	EngineVersion = FEngineVersion( Major, Minor, Patch, BuiltFromCL, BranchName );
}
