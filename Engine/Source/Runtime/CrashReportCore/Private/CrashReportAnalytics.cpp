// Copyright Epic Games, Inc. All Rights Reserved.

#include "CrashReportAnalytics.h"
#include "Misc/Guid.h"
#include "Misc/EngineBuildSettings.h"

#include "AnalyticsET.h"
#include "IAnalyticsProviderET.h"
#include "AnalyticsBuildType.h"

bool FCrashReportAnalytics::bIsInitialized;
TSharedPtr<IAnalyticsProviderET> FCrashReportAnalytics::Analytics;

/**
 * Default config func that essentially tells the crash reporter to disable analytics, unless
 * environment variables are set.
 */
FAnalyticsET::Config DefaultAnalyticsConfigFunc()
{
	FAnalyticsET::Config Config = FAnalyticsET::Config();
	// Check environment for settings. Both url and key needs to be set.
	FString Url = FPlatformMisc::GetEnvironmentVariable(TEXT("UE_CRC_TELEMETRY_URL"));
	FString Key = FPlatformMisc::GetEnvironmentVariable(TEXT("UE_CRC_TELEMETRY_KEY"));
	if (!Url.IsEmpty() && !Key.IsEmpty())
	{
		Config.APIServerET = Url;
		Config.APIKeyET = Key;
	}
	return Config;
}

/**
* Engine analytics config to initialize the crash reporter analytics provider.
* External code should bind this delegate if crash reporter analytics are desired,
* preferably in private code that won't be redistributed.
*/
TFunction<FAnalyticsET::Config()>& GetCrashReportAnalyticsConfigFunc()
{
	static TFunction<FAnalyticsET::Config()> Config = &DefaultAnalyticsConfigFunc;
	return Config;
}

/**
 * On-demand construction of the singleton. 
 */
IAnalyticsProviderET& FCrashReportAnalytics::GetProvider()
{
	checkf(bIsInitialized && Analytics.IsValid(), TEXT("FCrashReportAnalytics::GetProvider called outside of Initialize/Shutdown."));
	return *Analytics.Get();
}

void FCrashReportAnalytics::Initialize(const FString& EpicAccountId)
{
	TRACE_CPUPROFILER_EVENT_SCOPE_STR("FCrashReportAnalytics::Initialize");
	checkf(!bIsInitialized, TEXT("FCrashReportAnalytics::Initialize called more than once."));

	// Allow build machines to force CRC to enable internal telemetry.
	#if defined(CRC_TELEMETRY_URL) && defined(CRC_TELEMETRY_KEY) 
		FAnalyticsET::Config Config;
		Config.APIServerET = TEXT(CRC_TELEMETRY_URL);
		Config.APIKeyET = TEXT(CRC_TELEMETRY_KEY);
	#else
		FAnalyticsET::Config Config = GetCrashReportAnalyticsConfigFunc()();
	#endif

	if (!Config.APIServerET.IsEmpty())
	{
		// Connect the engine analytics provider (if there is a configuration delegate installed)
		Analytics = FAnalyticsET::Get().CreateAnalyticsProvider(Config);
		if( Analytics.IsValid() )
		{
			const FString AnalyticsEpicAccountId = EpicAccountId.IsEmpty() ? *FPlatformMisc::GetEpicAccountId() : *EpicAccountId;
			Analytics->SetUserID(FString::Printf(TEXT("%s|%s|%s"), *FPlatformMisc::GetLoginId(), *AnalyticsEpicAccountId, *FPlatformMisc::GetOperatingSystemId()));
			Analytics->StartSession();
		}
	}
	bIsInitialized = true;
}


void FCrashReportAnalytics::Shutdown()
{
	checkf(bIsInitialized, TEXT("FCrashReportAnalytics::Shutdown called outside of Initialize."));
	Analytics.Reset();
	bIsInitialized = false;
}
