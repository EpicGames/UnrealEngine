// Copyright Epic Games, Inc. All Rights Reserved.

#include "CrashReportCoreConfig.h"
#include "CrashReportCoreModule.h"
#include "Logging/LogMacros.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/App.h"
#include "Misc/EngineBuildSettings.h"

FCrashReportCoreConfig::FCrashReportCoreConfig()
	: DiagnosticsFilename( TEXT( "Diagnostics.txt" ) )
	, SectionName( TEXT( "CrashReportClient" ) )
{
	const bool bUnattended =
#if CRASH_REPORT_UNATTENDED_ONLY
		true;
#else
		FApp::IsUnattended();
#endif // CRASH_REPORT_UNATTENDED_ONLY

	if (!GConfig->GetString(*SectionName, TEXT("CrashReportClientVersion"), CrashReportClientVersion, GEngineIni))
	{
		CrashReportClientVersion = TEXT("0.0.0");
	}

	if (!GConfig->GetString( *SectionName, TEXT( "CrashReportReceiverIP" ), CrashReportReceiverIP, GEngineIni ))
	{
		// Use the default value (blank/disabled)
		CrashReportReceiverIP = TEXT("");
	}

	if (!GConfig->GetString(*SectionName, TEXT("DataRouterUrl"), DataRouterUrl, GEngineIni))
	{
#if defined CRC_DEFAULT_URL
		DataRouterUrl = TEXT(CRC_DEFAULT_URL);
#else
		DataRouterUrl = TEXT("");
#endif
	}

	if (FEngineBuildSettings::IsInternalBuild())
	{
		bAllowToBeContacted = true;
	}
	else if (!GConfig->GetBool(*SectionName, TEXT("bAllowToBeContacted"), bAllowToBeContacted, GEngineIni ))
	{
		// Default to true when unattended when config is missing. This is mostly for dedicated servers that do not have config files for CRC.
		if (bUnattended)
		{
			bAllowToBeContacted = true;
		}
	}

	if (!GConfig->GetBool(*SectionName, TEXT("bHideAllowToBeContactedOption"), bHideAllowToBeContactedOption, GEngineIni))
	{
		bHideAllowToBeContactedOption = false;
	}

	if (!GConfig->GetBool(*SectionName, TEXT("bSendLogFile"), bSendLogFile, GEngineIni ))
	{
		// Default to true when unattended when config is missing. This is mostly for dedicated servers that do not have config files for CRC.
		if (bUnattended)
		{
			bSendLogFile = true;
		}
	}

	if (!GConfig->GetInt(*SectionName, TEXT("UserCommentSizeLimit"), UserCommentSizeLimit, GEngineIni))
	{
		UserCommentSizeLimit = 4000;
	}

	if (!GConfig->GetBool(*SectionName, TEXT("bHideLogFilesOption"), bHideLogFilesOption, GEngineIni))
	{
		bHideLogFilesOption = false;
	}

	if (!GConfig->GetBool(*SectionName, TEXT("bHideRestartOption"), bHideRestartOption, GEngineIni))
	{
		bHideRestartOption = false;
	}

	if (!GConfig->GetBool(*SectionName, TEXT("bIsAllowedToCloseWithoutSending"), bIsAllowedToCloseWithoutSending, GEngineIni))
	{
		// Default to false (show the option) when config is missing.
		bIsAllowedToCloseWithoutSending = true;
	}

	if (!GConfig->GetBool(*SectionName, TEXT("bIsAllowedToSendWithoutDetailedInfo"), bIsAllowedToSendWithoutDetailedInfo, GEngineIni))
	{
		bIsAllowedToSendWithoutDetailedInfo = true;
	}

	if (!GConfig->GetBool(*SectionName, TEXT("bShowEndpointInTooltip"), bShowEndpointInTooltip, GEngineIni))
	{
		bShowEndpointInTooltip = false;
	}

	if (!GConfig->GetString(*SectionName, TEXT("CompanyName"), CompanyName, GEngineIni))
	{
#if defined(CRC_DEFAULT_COMPANY_NAME)
		CompanyName = TEXT(CRC_DEFAULT_COMPANY_NAME);
#else
		CompanyName = TEXT("");
#endif
	}

#if PLATFORM_WINDOWS
	if (!GConfig->GetBool(*SectionName, TEXT("bIsAllowedToCopyFilesToClipboard"), bIsAllowedToCopyFilesToClipboard, GEngineIni))
	{
		bIsAllowedToCopyFilesToClipboard = false;
	}
#endif

	ReadFullCrashDumpConfigurations();
}

void FCrashReportCoreConfig::SetAllowToBeContacted( bool bNewValue )
{
	bAllowToBeContacted = bNewValue;
	GConfig->SetBool( *SectionName, TEXT( "bAllowToBeContacted" ), bAllowToBeContacted, GEngineIni );
}

void FCrashReportCoreConfig::SetSendLogFile( bool bNewValue )
{
	bSendLogFile = bNewValue;
	GConfig->SetBool( *SectionName, TEXT( "bSendLogFile" ), bSendLogFile, GEngineIni );
}

void FCrashReportCoreConfig::ApplyProjectOverrides(const FString& ConfigFilePath)
{
	if (FPaths::FileExists(ConfigFilePath))
	{
		UE_LOGF(CrashReportCoreLog, Display, "Applying project settings from '%ls'.", *ConfigFilePath)
		FConfigFile ProjectConfig;
		ProjectConfig.Read(ConfigFilePath);
		SetProjectConfigOverrides(ProjectConfig);
	}
}

void FCrashReportCoreConfig::SetProjectConfigOverrides(const FConfigFile& InConfigFile)
{
	InConfigFile.GetString(*SectionName, TEXT("CrashReportClientVersion"), CrashReportClientVersion);
	InConfigFile.GetString(*SectionName, TEXT("CrashReportReceiverIP"), CrashReportReceiverIP);
	InConfigFile.GetString(*SectionName, TEXT("DataRouterUrl"), DataRouterUrl);
	InConfigFile.GetBool(*SectionName, TEXT("bAllowToBeContacted"), bAllowToBeContacted);
	InConfigFile.GetBool(*SectionName, TEXT("bHideAllowToBeContactedOption"), bHideAllowToBeContactedOption);
	InConfigFile.GetBool(*SectionName, TEXT( "bSendLogFile" ), bSendLogFile);
	InConfigFile.GetInt(*SectionName, TEXT("UserCommentSizeLimit"), UserCommentSizeLimit);
	InConfigFile.GetBool(*SectionName, TEXT("bHideLogFilesOption"), bHideLogFilesOption);
	InConfigFile.GetBool(*SectionName, TEXT("bHideRestartOption"), bHideRestartOption);
	InConfigFile.GetBool(*SectionName, TEXT("bIsAllowedToCloseWithoutSending"), bIsAllowedToCloseWithoutSending);
	InConfigFile.GetBool(*SectionName, TEXT("bIsAllowedToSendWithoutDetailedInfo"), bIsAllowedToSendWithoutDetailedInfo);
	InConfigFile.GetBool(*SectionName, TEXT("bShowEndpointInTooltip"), bShowEndpointInTooltip);
	InConfigFile.GetString(*SectionName, TEXT("CompanyName"), CompanyName);
#if PLATFORM_WINDOWS
	InConfigFile.GetBool(*SectionName, TEXT("bIsAllowedToCopyFilesToClipboard"), bIsAllowedToCopyFilesToClipboard);
#endif

	bool bUnixSocketEnabled = false;
	InConfigFile.GetBool(*SectionName, TEXT("bUnixSocketEnabled"), bUnixSocketEnabled); 
	if (bUnixSocketEnabled)
	{
		FString UnixSocketPath;
		InConfigFile.GetString(*SectionName, TEXT("DataRouterUnixSocketsPath"), UnixSocketPath);
		FString UnixSocketName;
		InConfigFile.GetString(*SectionName, TEXT("DataRouterUnixSocket"), UnixSocketName);
		DataRouterUnixSocket = UnixSocketPath / UnixSocketName;
	}
}

FString FCrashReportCoreConfig::GetFullCrashDumpLocationForBranch(const FString& BranchName) const
{
	for (const auto& It : FullCrashDumpConfigurations)
	{
		const bool bExactMatch = It.bExactMatch;
		const FString IterBranch = It.BranchName.Replace(TEXT("+"), TEXT("/"));
		if (bExactMatch && BranchName == IterBranch)
		{
			return It.Location;
		}
		else if (!bExactMatch && BranchName.Contains(IterBranch))
		{
			return It.Location;
		}
	}

	return TEXT( "" );
}

void FCrashReportCoreConfig::ReadFullCrashDumpConfigurations()
{
	auto GetKey = [this](const TCHAR* KeyName) -> FString
	{
		FString Result;
		if (!GConfig->GetString( *SectionName, KeyName, Result, GEngineIni ))
		{
			return TEXT( "" );
		}
		return MoveTemp(Result);
	};
	
	for (int32 NumEntries = 0;; ++NumEntries)
	{
		FString Branch = GetKey( *FString::Printf( TEXT( "FullCrashDump_%i_Branch" ), NumEntries ) );
		if (Branch.IsEmpty())
		{
			break;
		}

		const FString NetworkLocation = GetKey( *FString::Printf( TEXT( "FullCrashDump_%i_Location" ), NumEntries ) );
		const bool bExactMatch = !Branch.EndsWith( TEXT( "*" ) );
		Branch.ReplaceInline( TEXT( "*" ), TEXT( "" ) );

		FullCrashDumpConfigurations.Add( FFullCrashDumpEntry( Branch, NetworkLocation, bExactMatch ) );
	}
}

void FCrashReportCoreConfig::PrintSettingsToLog() const
{
	UE_LOGF(CrashReportCoreLog, Log, "CrashReportClientVersion=%ls", *CrashReportClientVersion);
	if (CrashReportReceiverIP.IsEmpty())
	{
		UE_LOGF(CrashReportCoreLog, Log, "CrashReportReceiver disabled");
	}
	else
	{
		UE_LOGF(CrashReportCoreLog, Log, "CrashReportReceiverIP: %ls", *CrashReportReceiverIP);
	}
	if (DataRouterUrl.IsEmpty())
	{
		UE_LOGF(CrashReportCoreLog, Log, "DataRouter disabled");
	}
	else
	{
		UE_LOGF(CrashReportCoreLog, Log, "DataRouterUrl: %ls", *DataRouterUrl);
	}
	for (auto Location : FullCrashDumpConfigurations)
	{
		UE_LOGF( CrashReportCoreLog, Log, "FullCrashDump: %ls, NetworkLocation: %ls, bExactMatch:%i", *Location.BranchName, *Location.Location, Location.bExactMatch);
	}
}
