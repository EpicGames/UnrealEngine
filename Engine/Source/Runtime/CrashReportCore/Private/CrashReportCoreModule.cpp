// Copyright Epic Games, Inc. All Rights Reserved.

#include "CrashReportCoreModule.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleManager.h"

IMPLEMENT_MODULE(FCrashReportCoreModule, CrashReportCore);
DEFINE_LOG_CATEGORY(CrashReportCoreLog);

void FCrashReportCoreModule::StartupModule()
{
}

void FCrashReportCoreModule::ShutdownModule()
{
}
