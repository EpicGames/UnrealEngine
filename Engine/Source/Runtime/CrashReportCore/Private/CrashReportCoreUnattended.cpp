// Copyright Epic Games, Inc. All Rights Reserved.

#include "CrashReportCoreUnattended.h"
#include "Containers/Ticker.h"
#include "CrashReportCoreConfig.h"
#include "CrashDescription.h"
#include "Stats/Stats.h"

FCrashReportCoreUnattended::FCrashReportCoreUnattended(FPlatformErrorReport& InErrorReport, bool InExitWhenComplete)
	: ReceiverUploader(FCrashReportCoreConfig::Get().GetReceiverAddress())
	, DataRouterUploader(FCrashReportCoreConfig::Get().GetDataRouterURL(), FCrashReportCoreConfig::Get().GetDataRouterUnixSocket())
	, ErrorReport(InErrorReport)
    , bUploadComplete(false)
	, bExitWhenComplete(InExitWhenComplete)
{
	ErrorReport.TryReadDiagnosticsFile();

	// Process the report synchronously
	ErrorReport.DiagnoseReport();

	// Update properties for the crash.
	ErrorReport.SetPrimaryCrashProperties( *FPrimaryCrashProperties::Get() );

	StartTicker();
}

bool FCrashReportCoreUnattended::Tick(float UnusedDeltaTime)
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FCrashReportClientUnattended_Tick);

	if (!FCrashUploadBase::IsInitialized())
	{
		FCrashUploadBase::StaticInitialize(ErrorReport);
	}

	if (ReceiverUploader.IsEnabled())
	{
		if (!ReceiverUploader.IsUploadCalled())
		{
			// Can be called only when we have all files.
			ReceiverUploader.BeginUpload(ErrorReport);
		}

		// IsWorkDone will always return true here (since ReceiverUploader can't finish until the diagnosis has been sent), but it
		//  has the side effect of joining the worker thread.
		if (!ReceiverUploader.IsFinished())
		{
			// More ticks, please
			return true;
		}
	}

	if (DataRouterUploader.IsEnabled())
	{
		if (!DataRouterUploader.IsUploadCalled())
		{
			// Can be called only when we have all files.
			DataRouterUploader.BeginUpload(ErrorReport);
		}

		// IsWorkDone will always return true here (since DataRouterUploader can't finish until the diagnosis has been sent), but it
		//  has the side effect of joining the worker thread.
		if (!DataRouterUploader.IsFinished())
		{
			// More ticks, please
			return true;
		}
	}

    bUploadComplete = true;
	return false;
}

void FCrashReportCoreUnattended::StartTicker()
{
	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FCrashReportCoreUnattended::Tick), 1.f);
}
