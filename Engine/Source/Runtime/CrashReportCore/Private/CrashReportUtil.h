// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformFile.h"


/**
 * Helper class for MakeDirectoryVisitor
 */
template <class FunctorType>
class FunctorDirectoryVisitor : public IPlatformFile::FDirectoryVisitor
{
public:
	/**
	 * Pass a directory or filename on to the user-provided functor
	 * @param FilenameOrDirectory Full path to a file or directory
	 * @param bIsDirectory Whether the path refers to a file or directory
	 * @return Whether to carry on iterating
	 */
	virtual bool Visit(const TCHAR* FilenameOrDirectory, bool bIsDirectory) override
	{
		return Functor(FilenameOrDirectory, bIsDirectory);
	}

	/**
	 * Move the provided functor into this object
	 */
	FunctorDirectoryVisitor(FunctorType&& FunctorInstance)
		: Functor(MoveTemp(FunctorInstance))
	{
	}

private:
	/** User-provided functor */
	FunctorType Functor;
};

/**
 * Convert a C++ functor into a IPlatformFile visitor object
 * @param FunctorInstance Function object to call for each directory item
 * @return Visitor object to be passed to platform directory visiting functions
 */
template <class Functor>
FunctorDirectoryVisitor<Functor> MakeDirectoryVisitor(Functor&& FunctorInstance)
{
	return FunctorDirectoryVisitor<Functor>(MoveTemp(FunctorInstance));
}
