// Copyright Epic Games, Inc. All Rights Reserved.
	
#include "CrashUpload.h"

#include "AnalyticsEventAttribute.h"
#include "CrashReportCoreModule.h"
#include "HAL/FileManager.h"
#include "Misc/Compression.h"
#include "Misc/FileHelper.h"
#include "Internationalization/Internationalization.h"
#include "Misc/Guid.h"
#include "Serialization/MemoryWriter.h"
#include "Containers/Ticker.h"
#include "CrashReportCoreConfig.h"

#include "Interfaces/IHttpResponse.h"
#include "HttpModule.h"
#include "GenericPlatform/GenericPlatformHttp.h"
#include "PendingReports.h"
#include "CrashDescription.h"
#include "Misc/EngineBuildSettings.h"
#include "Misc/CommandLine.h"

#include "Stats/Stats.h"

// Switched off CRR upload - Jun 2016
#define PRIMARY_UPLOAD_RECEIVER 0
#define PRIMARY_UPLOAD_DATAROUTER 1

#define LOCTEXT_NAMESPACE "CrashReportClient"

namespace CrashUploadDefs
{
	const float PingTimeoutSeconds = 5.f;

	const FString APIKey("CrashReporter");
	const FString AppEnvironmentInternal("Dev");
	const FString AppEnvironmentExternal("Release");
	const FString UploadType("crashreports");
}

enum class ECompressedCrashFileHeader
{
	MAGIC = 0x7E1B83C1,
};

struct FCompressedCrashFile : FNoncopyable
{
	int32 CurrentFileIndex;
	FString Filename;
	TArray<uint8> Filedata;

	FCompressedCrashFile(int32 InCurrentFileIndex, const FString& InFilename, const TArray<uint8>& InFiledata)
		: CurrentFileIndex(InCurrentFileIndex)
		, Filename(InFilename)
		, Filedata(InFiledata)
	{
	}

	/** Serialization operator. */
	friend FArchive& operator << (FArchive& Ar, FCompressedCrashFile& Data)
	{
		Ar << Data.CurrentFileIndex;
		Data.Filename.SerializeAsANSICharArray(Ar, 260);
		Ar << Data.Filedata;
		return Ar;
	}
};

struct FCompressedHeader
{
	FString DirectoryName;
	FString FileName;
	int32 UncompressedSize;
	int32 FileCount;

	/** Serialization operator. */
	friend FArchive& operator << (FArchive& Ar, FCompressedHeader& Data)
	{
		// The 'CR1' marker prevents the data router backend to fallback to a backward compatibility version where
		// a buggy/incomplete header was written at the beginning of the stream and the correct/valid one at the end.
		uint8 Version[] = {'C', 'R', '1'};
		Ar.Serialize(Version, sizeof(Version));

		Data.DirectoryName.SerializeAsANSICharArray(Ar, 260);
		Data.FileName.SerializeAsANSICharArray(Ar, 260);
		Ar << Data.UncompressedSize;
		Ar << Data.FileCount;
		return Ar;
	}
};

struct FCompressedData
{
	TArray<uint8> Data;
	int32 CompressedSize;
	int32 UncompressedSize;
	int32 FileCount;
};

bool FCrashUploadBase::bInitialized = false;
TArray<FString> FCrashUploadBase::PendingReportDirectories;
TArray<FString> FCrashUploadBase::FailedReportDirectories;

FCrashUploadBase::FCrashUploadBase()
	: bUploadCalled(false)
	, State(EUploadState::NotSet)
	, PauseState(EUploadState::Ready)
	, PendingReportDirectoryIndex(0)
{

}

FCrashUploadBase::~FCrashUploadBase()
{
	UE_LOGF(CrashReportCoreLog, Log, "Final state (Receiver) = %ls", ToString(State));
}

void FCrashUploadBase::StaticInitialize(const FPlatformErrorReport& PlatformErrorReport)
{
	FPendingReports PendingReports;
	//PendingReports.Add(*PlatformErrorReport.GetReportDirectoryLeafName());
	PendingReportDirectories = PendingReports.GetReportDirectories();
	PendingReports.Clear();
	PendingReports.Save();
}

void FCrashUploadBase::StaticShutdown()
{
	// Write failed uploads back to disk
	FPendingReports ReportsForNextTime;
	for (const FString& FailedReport : FailedReportDirectories)
	{
		ReportsForNextTime.Add(FailedReport);
	}
	ReportsForNextTime.Save();
}

bool FCrashUploadBase::CompressData(const TArray<FString>& InPendingFiles, FCompressedData& OutCompressedData, TArray<uint8>& OutPostData, FCompressedHeader* OptionalHeader)
{
	UE_LOGF(CrashReportCoreLog, Log, "CompressAndSendData have %d pending files", InPendingFiles.Num());

	// Compress all files into one archive.
	const int32 BufferSize = 32 * 1024 * 1024;

	TArray<uint8> UncompressedData;
	UncompressedData.Reserve(BufferSize);
	FMemoryWriter MemoryWriter(UncompressedData, false, true);

	if (OptionalHeader != nullptr)
	{
		// Write dummy to fill correct size
		MemoryWriter << *OptionalHeader;
	}

	int32 CurrentFileIndex = 0;

	const FString FullCrashDumpLocation = FPrimaryCrashProperties::Get()->FullCrashDumpLocation.AsString();

	bool bIsFullDumpCrash = (FPrimaryCrashProperties::Get()->CrashDumpMode == ECrashDumpMode::FullDump ||
		FPrimaryCrashProperties::Get()->CrashDumpMode == ECrashDumpMode::FullDumpAlways) &&
		FPrimaryCrashProperties::Get()->CrashVersion >= ECrashDescVersions::VER_3_CrashContext;

	// Loop to keep trying files until a send succeeds or we run out of files
	for (const FString& PathOfFileToUpload : InPendingFiles)
	{
		const FString Filename = FPaths::GetCleanFilename(PathOfFileToUpload);

		const bool bIsFullDumpFile = Filename == FGenericCrashContext::UEMinidumpName && bIsFullDumpCrash;
		const bool bValidFullDumpForCopy = bIsFullDumpFile && !FullCrashDumpLocation.IsEmpty();

		if (bIsFullDumpFile)
		{
			if (bValidFullDumpForCopy)
			{
				const FString DestinationPath = FullCrashDumpLocation / FGenericCrashContext::UEMinidumpName;
				const bool bCreated = IFileManager::Get().MakeDirectory(*FullCrashDumpLocation, true);
				if (!bCreated)
				{
					UE_LOGF(CrashReportCoreLog, Error, "Couldn't create directory for full crash dump %ls", *DestinationPath);
				}
				else
				{
					UE_LOGF(CrashReportCoreLog, Warning, "Copying full crash minidump to %ls", *DestinationPath);
					IFileManager::Get().Copy(*DestinationPath, *PathOfFileToUpload, false);
				}
			}

			UE_LOGF(CrashReportCoreLog, Log, "Skipping upload of full crash dump");
			continue;
		}

		if (!FFileHelper::LoadFileToArray(OutPostData, *PathOfFileToUpload))
		{
			UE_LOGF(CrashReportCoreLog, Warning, "Failed to load crash report file");
			continue;
		}

		const bool bSkipLogFile = !FCrashReportCoreConfig::Get().GetSendLogFile() && PathOfFileToUpload.EndsWith(TEXT(".log"));
		if (bSkipLogFile)
		{
			UE_LOGF(CrashReportCoreLog, Warning, "Skipping the %ls", *Filename);
			continue;
		}

		// Disabled due to issues with Mac not using the crash context.
		/*
		// Skip old WERInternalMetadata.
		const bool bSkipXMLFile = PathOfFileToUpload.EndsWith( TEXT( ".xml" ) );
		if (bSkipXMLFile)
		{
		UE_LOGF( CrashReportCoreLog, Warning, "Skipping the %ls", *Filename );
		continue;
		}*/

		// Skip old Report.wer file.
		const bool bSkipWERFile = PathOfFileToUpload.Contains(TEXT("Report.wer"));
		if (bSkipWERFile)
		{
			UE_LOGF(CrashReportCoreLog, Warning, "Skipping the %ls", *Filename);
			continue;
		}

		// Skip old diagnostics.txt file, all data is stored in the CrashContext.runtime-xml
		// Disabled due to issues with Mac not using the crash context.
		/*
		const bool bSkipDiagnostics = Filename == FCrashReportClientConfig::Get().GetDiagnosticsFilename();
		if (bSkipDiagnostics)
		{
		UE_LOGF( CrashReportCoreLog, Warning, "Skipping the %ls", *Filename );
		continue;
		}*/

		UE_LOGF(CrashReportCoreLog, Log, "CompressAndSendData compressing %d bytes ('%ls')", OutPostData.Num(), *PathOfFileToUpload);

		FCompressedCrashFile FileToCompress(CurrentFileIndex, Filename, OutPostData);
		CurrentFileIndex++;

		MemoryWriter << FileToCompress;
	}

	if (OptionalHeader != nullptr)
	{
		FMemoryWriter MemoryHeaderWriter(UncompressedData);

		OptionalHeader->UncompressedSize = UncompressedData.Num();
		OptionalHeader->FileCount = CurrentFileIndex;

		MemoryHeaderWriter << *OptionalHeader;
	}

	int UncompressedSize = UncompressedData.Num();

	TArray<uint8> CompressedDataRaw;
	CompressedDataRaw.AddUninitialized(UncompressedSize);

	OutCompressedData.FileCount = CurrentFileIndex;
	OutCompressedData.CompressedSize = UncompressedSize;
	OutCompressedData.UncompressedSize = UncompressedSize;
	const bool bResult = FCompression::CompressMemory(NAME_Zlib, CompressedDataRaw.GetData(), OutCompressedData.CompressedSize, UncompressedData.GetData(), OutCompressedData.UncompressedSize);
	if (bResult)
	{
		// Copy compressed data into the array.
		OutCompressedData.Data.Append(CompressedDataRaw.GetData(), OutCompressedData.CompressedSize);
	}

	return bResult;
}

const TCHAR* FCrashUploadBase::ToString(EUploadState::Type State)
{
	switch (State)
	{
	case EUploadState::PingingServer:
		return TEXT("PingingServer");

	case EUploadState::Ready:
		return TEXT("Ready");

	case EUploadState::CheckingReport:
		return TEXT("CheckingReport");

	case EUploadState::CheckingReportDetail:
		return TEXT("CheckingReportDetail");

	case EUploadState::CompressAndSendData:
		return TEXT("SendingFiles");

	case EUploadState::WaitingToPostReportComplete:
		return TEXT("WaitingToPostReportComplete");

	case EUploadState::PostingReportComplete:
		return TEXT("PostingReportComplete");

	case EUploadState::Finished:
		return TEXT("Finished");

	case EUploadState::ServerNotAvailable:
		return TEXT("ServerNotAvailable");

	case EUploadState::UploadError:
		return TEXT("UploadError");

	case EUploadState::Cancelled:
		return TEXT("Cancelled");

	default:
		break;
	}

	return TEXT("Unknown UploadState value");
}

void FCrashUploadBase::SetCurrentState(EUploadState::Type InState)
{
	if (State == EUploadState::NotSet)
	{
		UE_LOGF(CrashReportCoreLog, Log, "Initial state = %ls", ToString(State));
	}
	else
	{
		UE_LOGF(CrashReportCoreLog, Log, "State change from %ls to %ls", ToString(State), ToString(InState));
	}

	State = InState;

	switch (State)
	{
	default:
		break;

	case EUploadState::PingingServer:
		UploadStateText = LOCTEXT("PingingServer", "Pinging server");
		break;

	case EUploadState::Ready:
		UploadStateText = LOCTEXT("UploaderReady", "Ready to send to server");
		break;

	case EUploadState::ServerNotAvailable:
		UploadStateText = LOCTEXT("ServerNotAvailable", "Server not available - report will be stored for later upload");
		break;
	}
}

void FCrashUploadBase::AddReportToFailedList() const
{
	if (PendingFiles.Num() > 0)
	{
		FailedReportDirectories.AddUnique(ErrorReport.GetReportDirectory());
	}
}

void FCrashUploadBase::CleanCrashReportDirectory(const FString& CrashReportDirectory)
{
	// Clean crash report folder if requested
	if (FParse::Param(FCommandLine::Get(), TEXT("CleanCrashReports")))
	{
		// Check that the crash directory is valid and resides in /Saved/Crashes
		FString NormalizedReportDirectory = CrashReportDirectory;
		FPaths::NormalizeDirectoryName(NormalizedReportDirectory);
		if (NormalizedReportDirectory.Contains(TEXT("/Saved/Crashes/")))
		{
			UE_LOGF(CrashReportCoreLog, Log, "Removing crash report directory %ls", *CrashReportDirectory);
			IFileManager& FileManager = IFileManager::Get();
			FileManager.DeleteDirectory(*CrashReportDirectory, false, true);
		}
	}
}


// FCrashUploadToReceiver //////////////////////////////////////////////////////

FCrashUploadToReceiver::FCrashUploadToReceiver(const FString& InReceiverAddress)
	: UrlPrefix(InReceiverAddress.IsEmpty() ? TEXT("") : InReceiverAddress / TEXT("CrashReporter"))
{
	if (!UrlPrefix.IsEmpty())
	{
		// Sending to receiver
		SendPingRequest();
	}
	else
	{
		SetCurrentState(EUploadState::Disabled);
	}
}

FCrashUploadToReceiver::~FCrashUploadToReceiver()
{
}

bool FCrashUploadToReceiver::PingTimeout(float DeltaTime)
{
    QUICK_SCOPE_CYCLE_COUNTER(STAT_FCrashUploadToReceiver_PingTimeout);

	if (EUploadState::PingingServer == State)
	{
		SetCurrentState(EUploadState::ServerNotAvailable);

		// PauseState will be Ready if user has not yet decided to send the report
		if (PauseState > EUploadState::Ready)
		{
			AddReportToFailedList();
		}
	}

	// One-shot
	return false;
}

void FCrashUploadToReceiver::BeginUpload(const FPlatformErrorReport& PlatformErrorReport)
{
	bUploadCalled = true;

	ErrorReport = PlatformErrorReport;
	PendingFiles = FPlatformErrorReport( ErrorReport.GetReportDirectory() ).GetFilesToUpload();
	UE_LOGF(CrashReportCoreLog, Log, "Got %d pending files to upload from '%ls'", PendingFiles.Num(), *ErrorReport.GetReportDirectoryLeafName());

	PauseState = EUploadState::Finished;
	if (State == EUploadState::Ready)
	{
		BeginUploadImpl();
	}
	else if (State == EUploadState::ServerNotAvailable)
	{
		AddReportToFailedList();
	}
}

bool FCrashUploadToReceiver::SendCheckReportRequest()
{
	FString XMLString;

	auto Request = CreateHttpRequest();
	if (State == EUploadState::CheckingReport)
	{
#if PRIMARY_UPLOAD_RECEIVER
		// first stage of any upload to CRR so send analytics
		FPrimaryCrashProperties::Get()->SendPreUploadAnalytics();
#endif
		AssignReportIdToPostDataBuffer();
		Request->SetURL(UrlPrefix / TEXT("CheckReport"));
		Request->SetHeader(TEXT("Content-Type"), TEXT("text/plain; charset=us-ascii"));

		UE_LOGF( CrashReportCoreLog, Log, "Sending HTTP request: %ls", *Request->GetURL() );
	}
	else
	{
		// This part is Windows-specific on the server
		ErrorReport.LoadWindowsReportXmlFile( XMLString );

		// Convert the XMLString into the UTF-8.
		FTCHARToUTF8 Converter( (const TCHAR*)*XMLString, XMLString.Len() );
		const int32 Length = Converter.Length();
		PostData.Reset( Length );
		PostData.AddUninitialized( Length );
		CopyAssignItems( (ANSICHAR*)PostData.GetData(), Converter.Get(), Length );

		Request->SetURL(UrlPrefix / TEXT("CheckReportDetail"));
		Request->SetHeader(TEXT("Content-Type"), TEXT("text/plain; charset=utf-8"));

		UE_LOGF( CrashReportCoreLog, Log, "Sending HTTP request: %ls", *Request->GetURL() );
	}

	UE_LOGF( CrashReportCoreLog, Log, "PostData Num: %i", PostData.Num() );
	Request->SetVerb(TEXT("POST"));
	Request->SetContent(PostData);

	return Request->ProcessRequest();
}

void FCrashUploadToReceiver::CompressAndSendData()
{
	FCompressedData CompressedData;
	if (!CompressData(PendingFiles, CompressedData, PostData))
	{
		UE_LOGF(CrashReportCoreLog, Warning, "Couldn't compress the crash report files");
		SetCurrentState(EUploadState::Cancelled);
		return;
	}

	PendingFiles.Empty();

	const FString Filename = ErrorReport.GetReportDirectoryLeafName() + TEXT(".uecrash");

	// Set up request for upload
	auto Request = CreateHttpRequest();
	Request->SetVerb(TEXT("POST"));
	Request->SetHeader(TEXT("Content-Type"), TEXT("application/octet-stream"));
	Request->SetURL(UrlPrefix / TEXT("UploadReportFile"));
	Request->SetContent(CompressedData.Data);
	Request->SetHeader(TEXT("DirectoryName"), *ErrorReport.GetReportDirectoryLeafName());
	Request->SetHeader(TEXT("FileName"), Filename);
	Request->SetHeader(TEXT("FileLength"), TTypeToString<int32>::ToString(CompressedData.Data.Num()) );
	Request->SetHeader(TEXT("CompressedSize"), TTypeToString<int32>::ToString(CompressedData.CompressedSize) );
	Request->SetHeader(TEXT("UncompressedSize"), TTypeToString<int32>::ToString(CompressedData.UncompressedSize) );
	Request->SetHeader(TEXT("NumberOfFiles"), TTypeToString<int32>::ToString(CompressedData.FileCount) );
	UE_LOGF( CrashReportCoreLog, Log, "Sending HTTP request: %ls, Payload size: %d", *Request->GetURL(), CompressedData.Data.Num());

	if (Request->ProcessRequest())
	{
		return;
	}
	else
	{
		UE_LOGF(CrashReportCoreLog, Warning, "Failed to send file upload request");
		SetCurrentState(EUploadState::Cancelled);
	}	
}

void FCrashUploadToReceiver::AssignReportIdToPostDataBuffer()
{
	FString ReportDirectoryName = *ErrorReport.GetReportDirectoryLeafName();
	const int32 DirectoryNameLength = ReportDirectoryName.Len();
	PostData.SetNum(DirectoryNameLength);
	for (int32 Index = 0; Index != DirectoryNameLength; ++Index)
	{
		PostData[Index] = ReportDirectoryName[Index];
	}
}

void FCrashUploadToReceiver::PostReportComplete()
{
	if (PauseState == EUploadState::PostingReportComplete)
	{
		// Wait for confirmation
		SetCurrentState(EUploadState::WaitingToPostReportComplete);
		return;
	}

	AssignReportIdToPostDataBuffer();
	
	auto Request = CreateHttpRequest();
	Request->SetVerb( TEXT( "POST" ) );
	Request->SetURL(UrlPrefix / TEXT("UploadComplete"));
	Request->SetHeader( TEXT( "Content-Type" ), TEXT( "text/plain; charset=us-ascii" ) );
	Request->SetContent(PostData);
	UE_LOGF( CrashReportCoreLog, Log, "Sending HTTP request: %ls, Payload size: %d", *Request->GetURL(), PostData.Num());

	if (Request->ProcessRequest())
	{
#if PRIMARY_UPLOAD_RECEIVER
		// completed upload to CRR so send analytics
		FPrimaryCrashProperties::Get()->SendPostUploadAnalytics(0.0, false, 0, 0);
#endif
		SetCurrentState(EUploadState::PostingReportComplete);
	}
	else
	{
		CheckPendingReportsForFilesToUpload();
	}
}

void FCrashUploadToReceiver::OnProcessRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
{
	UE_LOGF(CrashReportCoreLog, Log, "OnProcessRequestComplete(), State=%ls bSucceeded=%i", ToString(State), (int32)bSucceeded );
	switch (State)
	{
	default:
		// May get here if response is received after time-out has passed
		break;

	case EUploadState::PingingServer:
		if (bSucceeded)
		{
			OnPingSuccess();
		}
		else
		{
			PingTimeout(0);
		}
		break;

	case EUploadState::CheckingReport:
	case EUploadState::CheckingReportDetail:
		{
			bool bCheckedOkay = false;

			if (!bSucceeded || !ParseServerResponse(HttpResponse, bCheckedOkay))
			{
				if (!bSucceeded)
				{
					UE_LOGF(CrashReportCoreLog, Warning, "Request to server failed");
				}
				else
				{
					UE_LOGF(CrashReportCoreLog, Warning, "Did not get a valid server response.");
				}

				// Failed to check with the server - skip this report for now
				AddReportToFailedList();
				CheckPendingReportsForFilesToUpload();
			}
			else if (!bCheckedOkay)
			{
				// Server rejected the report
				UE_LOGF(CrashReportCoreLog, Warning, "Did not get a valid server response.");
				CheckPendingReportsForFilesToUpload();
			}
			else
			{
				SetCurrentState(EUploadState::CompressAndSendData);
				CompressAndSendData();
			}
		}
		break;

	case EUploadState::CompressAndSendData:
		if (!bSucceeded)
		{
			UE_LOGF(CrashReportCoreLog, Warning, "File upload failed to receiver");
			AddReportToFailedList();
			SetCurrentState(EUploadState::Cancelled);
		}
		else
		{
			PostReportComplete();
		}
		break;

	case EUploadState::PostingReportComplete:
		CheckPendingReportsForFilesToUpload();
		break;
	}
}

void FCrashUploadToReceiver::OnPingSuccess()
{
	if (PauseState > EUploadState::Ready)
	{
		BeginUploadImpl();
	}
	else
	{
		// Await instructions
		SetCurrentState(EUploadState::Ready);
	}
}

void FCrashUploadToReceiver::CheckPendingReportsForFilesToUpload()
{
	SetCurrentState(EUploadState::CheckingReport);

	for (; PendingReportDirectoryIndex < PendingReportDirectories.Num(); PendingReportDirectoryIndex++)
	{
		ErrorReport = FPlatformErrorReport(PendingReportDirectories[PendingReportDirectoryIndex]);
		PendingFiles = ErrorReport.GetFilesToUpload();

		if (PendingFiles.Num() > 0 && SendCheckReportRequest())
		{
			return;
		}
	}

	// Nothing left to upload
	UE_LOGF(CrashReportCoreLog, Log, "All uploads done");
	SetCurrentState(EUploadState::Finished);
}

void FCrashUploadToReceiver::BeginUploadImpl()
{
	SetCurrentState(EUploadState::CheckingReport);
	if (!SendCheckReportRequest())
	{
		CheckPendingReportsForFilesToUpload();
	}
}

TSharedRef<IHttpRequest, ESPMode::ThreadSafe> FCrashUploadToReceiver::CreateHttpRequest()
{
	auto Request = FHttpModule::Get().CreateRequest();
	Request->OnProcessRequestComplete().BindRaw(this, &FCrashUploadToReceiver::OnProcessRequestComplete);
	return Request;
}

void FCrashUploadToReceiver::SendPingRequest()
{
	SetCurrentState(EUploadState::PingingServer);

	auto Request = CreateHttpRequest();
	Request->SetVerb(TEXT("GET"));
	Request->SetURL(UrlPrefix / TEXT("Ping"));
	UE_LOGF( CrashReportCoreLog, Log, "Sending HTTP request: %ls", *Request->GetURL() );

	if (Request->ProcessRequest())
	{
		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FCrashUploadToReceiver::PingTimeout), CrashUploadDefs::PingTimeoutSeconds);
	}
	else
	{
		PingTimeout(0);
	}
}

bool FCrashUploadToReceiver::ParseServerResponse(FHttpResponsePtr Response, bool& OutValidReport)
{
	if (!Response.IsValid())
	{
		return false;
	}
	// Turn the snippet into a complete XML document, to keep the XML parser happy
	FXmlFile ParsedResponse(FString("<Root>") + Response->GetContentAsString() + TEXT("</Root>"), EConstructMethod::ConstructFromBuffer);
	UE_LOGF(CrashReportCoreLog, Log, "Response->GetContentAsString(): '%ls'", *Response->GetContentAsString());
	if (!ParsedResponse.IsValid())
	{
		UE_LOGF(CrashReportCoreLog, Log, "Invalid response!");
		OutValidReport = false;
		return false;
	}
	if (auto ResultNode = ParsedResponse.GetRootNode()->FindChildNode(TEXT("CrashReporterResult")))
	{
		UE_LOGF(CrashReportCoreLog, Log, "ResultNode->GetAttribute(TEXT(\"bSuccess\")) = %ls", *ResultNode->GetAttribute(TEXT("bSuccess")));
		OutValidReport = ResultNode->GetAttribute(TEXT("bSuccess")) == TEXT("true");
		return true;
	}
	UE_LOGF(CrashReportCoreLog, Log, "Could not find CrashReporterResult");
	OutValidReport = false;
	return false;
}


// FCrashUploadToDataRouter //////////////////////////////////////////////////////

FCrashUploadToDataRouter::FCrashUploadToDataRouter(const FString& InDataRouterUrl, const FString& InOptionalDataRouterUnixSocket)
	: DataRouterUrl(InDataRouterUrl)
	, DataRouterUnixSocket(InOptionalDataRouterUnixSocket)
	, Timer(Duration)
{
	if (!DataRouterUrl.IsEmpty())
	{
		SetCurrentState(EUploadState::Ready);
	}
	else
	{
		SetCurrentState(EUploadState::Disabled);
	}

#if PRIMARY_UPLOAD_DATAROUTER
	// first stage of any upload to DR so send analytics
	FPrimaryCrashProperties::Get()->SendPreUploadAnalytics();
#endif
}

FCrashUploadToDataRouter::~FCrashUploadToDataRouter()
{
	Timer.Stop();
#if PRIMARY_UPLOAD_DATAROUTER
	// completed upload to DR so send analytics
	FPrimaryCrashProperties::Get()->SendPostUploadAnalytics(Duration, bResult, ResponseCode, PayloadSize, ReportCount);
#endif		
}

void FCrashUploadToDataRouter::BeginUpload(const FPlatformErrorReport& PlatformErrorReport)
{
	bUploadCalled = true;

	Timer.Start();

	ErrorReport = PlatformErrorReport;
	PendingFiles = FPlatformErrorReport(ErrorReport.GetReportDirectory()).GetFilesToUpload();
	UE_LOGF(CrashReportCoreLog, Log, "Got %d pending files to upload from '%ls'", PendingFiles.Num(), *ErrorReport.GetReportDirectoryLeafName());

	PauseState = EUploadState::Finished;
	if (State == EUploadState::Ready)
	{
		SetCurrentState(EUploadState::CompressAndSendData);
		CompressAndSendData();
	}
}

void FCrashUploadToDataRouter::BeginUploadOptionalAttachments(const FPlatformErrorReport& PlatformErrorReport)
{
	ErrorReport = PlatformErrorReport;
	PendingFiles = FPlatformErrorReport(ErrorReport.GetReportDirectory()).GetOptionalAttachmentsToUpload();
	UE_LOGF(CrashReportCoreLog, Log, "Got %d pending optional attachment files to upload", PendingFiles.Num());

	PauseState = EUploadState::Finished;
	// We should only upload optional attachments when the base report was successfully uploaded (EUploadState::Finished).
	if (State == EUploadState::Finished)
	{
		SetCurrentState(EUploadState::CompressAndSendData);
		CompressAndSendData();
	}
}

void FCrashUploadToDataRouter::CompressAndSendData()
{
	FCompressedHeader CompressedHeader;
	CompressedHeader.DirectoryName = ErrorReport.GetReportDirectoryLeafName();
	CompressedHeader.FileName = ErrorReport.GetReportDirectoryLeafName() + TEXT(".uecrash");

	FCompressedData CompressedData;
	if (!CompressData(PendingFiles, CompressedData, PostData, &CompressedHeader))
	{
		UE_LOGF(CrashReportCoreLog, Warning, "Couldn't compress the crash report files");
		SetCurrentState(EUploadState::Cancelled);
		return;
	}

	PayloadSize += CompressedData.Data.Num();
	++ReportCount;

	PendingFiles.Empty();

	FString UserId = FString::Printf(TEXT("%s|%s|%s"), *FPlatformMisc::GetLoginId(), *FPlatformMisc::GetEpicAccountId(), *FPlatformMisc::GetOperatingSystemId());

	FString UrlParams = FString::Printf(TEXT("?AppID=%s&AppVersion=%s&AppEnvironment=%s&UploadType=%s&UserID=%s"),
		*FGenericPlatformHttp::UrlEncode(CrashUploadDefs::APIKey),
		*FGenericPlatformHttp::UrlEncode(FEngineVersion::Current().ToString()),
		*FGenericPlatformHttp::UrlEncode(FEngineBuildSettings::IsInternalBuild() ? CrashUploadDefs::AppEnvironmentInternal : CrashUploadDefs::AppEnvironmentExternal),
		*FGenericPlatformHttp::UrlEncode(CrashUploadDefs::UploadType),
		*FGenericPlatformHttp::UrlEncode(UserId));

	// Set up request for upload
	auto Request = CreateHttpRequest();
	Request->SetVerb(TEXT("POST"));
	Request->SetHeader(TEXT("Content-Type"), TEXT("application/octet-stream"));
	Request->SetURL(DataRouterUrl + UrlParams);
#if UE_HTTP_SUPPORT_UNIX_SOCKET
	if (!DataRouterUnixSocket.IsEmpty())
	{
		Request->SetOption(HttpRequestOptions::UnixSocketPath, DataRouterUnixSocket);
	}
#endif
	Request->SetContent(CompressedData.Data);
	UE_LOGF(CrashReportCoreLog, Log, "Sending HTTP request: %ls, Payload size: %d", *Request->GetURL(), CompressedData.Data.Num());

	if (!Request->ProcessRequest())
	{
		UE_LOGF(CrashReportCoreLog, Warning, "Failed to send file upload request");
		SetCurrentState(EUploadState::Cancelled);
	}
}

TSharedRef<IHttpRequest, ESPMode::ThreadSafe> FCrashUploadToDataRouter::CreateHttpRequest()
{
	auto Request = FHttpModule::Get().CreateRequest();
	Request->OnProcessRequestComplete().BindRaw(this, &FCrashUploadToDataRouter::OnProcessRequestComplete);
	return Request;
}

void FCrashUploadToDataRouter::OnProcessRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded)
{
	bResult = bSucceeded;
	ResponseCode = HttpResponse.IsValid() ? HttpResponse->GetResponseCode() : -1;

	UE_LOGF(CrashReportCoreLog, Log, "OnProcessRequestComplete(), State=%ls Response=%i ConnectedSuccesfully=%i", ToString(State), ResponseCode, (int32)bSucceeded);
	
	if (!EHttpResponseCodes::IsOk(ResponseCode))
	{
		const FText Description = GetDescription((EHttpResponseCodes::Type)ResponseCode);
		UE_LOGF(CrashReportCoreLog, Error, "Failed to send crash report. Server returned error code %u (%ls).", ResponseCode, *Description.ToString());
		SetCurrentState(EUploadState::Cancelled);
	}
	
	switch (State)
	{
	default:
		// May get here if response is received after time-out has passed
		break;

	case EUploadState::CompressAndSendData:
		if (!bSucceeded)
		{
			UE_LOGF(CrashReportCoreLog, Warning, "File upload failed to data router");
			AddReportToFailedList();
			SetCurrentState(EUploadState::Cancelled);
		}
		else
		{
			// Successfully submitted crash report
			CleanCrashReportDirectory(ErrorReport.GetReportDirectory());
			CheckPendingReportsForFilesToUpload();
		}
		break;
	}
}

void FCrashUploadToDataRouter::CheckPendingReportsForFilesToUpload()
{
	if (!PendingReportDirectories.IsEmpty())
	{
		UE_LOGF(CrashReportCoreLog, Log, "Found additional pending reports");
		SetCurrentState(EUploadState::CompressAndSendData);
		for (; PendingReportDirectoryIndex < PendingReportDirectories.Num(); PendingReportDirectoryIndex++)
		{
			ErrorReport = FPlatformErrorReport(PendingReportDirectories[PendingReportDirectoryIndex]);
			PendingFiles = ErrorReport.GetFilesToUpload();

			if (PendingFiles.Num() > 0)
			{
				CompressAndSendData();
				return;
			}
		}
	}

	// Nothing left to upload
	UE_LOGF(CrashReportCoreLog, Log, "All uploads done");
	SetCurrentState(EUploadState::Finished);
}

#undef LOCTEXT_NAMESPACE
