// Copyright Epic Games, Inc. All Rights Reserved.

#include "GenericErrorReport.h"
#include "HAL/PlatformProcess.h"
#include "HAL/PlatformFileManager.h"
#include "UObject/NameTypes.h"
#include "Logging/LogMacros.h"
#include "Misc/EngineBuildSettings.h"
#include "Misc/Parse.h"
#include "Misc/FileHelper.h"
#include "Internationalization/Text.h"
#include "CrashReportCoreConfig.h"
#include "Modules/ModuleManager.h"
#include "CrashReportUtil.h"


#if WITH_CRASHREPORTER
#include "CrashDebugHelper.h"
#include "CrashDebugHelperModule.h"
#endif

// ----------------------------------------------------------------
// Helpers

namespace
{
	/** Enum specifying a particular part of a crash report text file */
	namespace EReportSection
	{
		enum Type
		{
			CallStack,
			SourceContext,
			Other
		};
	}
}


// ----------------------------------------------------------------
// FGenericErrorReport

FGenericErrorReport::FGenericErrorReport(const FString& Directory)
	: ReportDirectory(Directory)
	, OptionalAttachmentsDirectory(FPaths::Combine(Directory, FGenericCrashContext::CrashReportOptionalAttachmentsDirName))
	, bValidCallstack(true)
{
	auto FilenamesVisitor = MakeDirectoryVisitor([this](const TCHAR* FilenameOrDirectory, bool bIsDirectory) {
		if (!bIsDirectory)
		{
			ReportFilenames.Push(FPaths::GetCleanFilename(FilenameOrDirectory));
		}
		return true;
	});

	auto OptionalFilenamesVisitor = MakeDirectoryVisitor([this](const TCHAR* FilenameOrDirectory, bool bIsDirectory) {
		if (!bIsDirectory)
		{
			OptionalAttachmentsFilenames.Push(FPaths::GetCleanFilename(FilenameOrDirectory));
		}
		return true;
	});

	FPlatformFileManager::Get().GetPlatformFile().IterateDirectory(*ReportDirectory, FilenamesVisitor);
	FPlatformFileManager::Get().GetPlatformFile().IterateDirectory(*OptionalAttachmentsDirectory, OptionalFilenamesVisitor);
}

bool FGenericErrorReport::SetUserComment(const FText& UserComment)
{
	const bool bAllowToBeContacted = FCrashReportCoreConfig::Get().GetAllowToBeContacted();

	const FString UserName1 = FPlatformProcess::UserName( false );
	const FString UserName2 = FPlatformProcess::UserName( true );
	const TCHAR* Anonymous = TEXT( "Anonymous" );

	FPrimaryCrashProperties::Get()->UserDescription = UserComment.ToString();

	// Load the file and remove all PII if not an internal build
	const bool bRemovePersonalData = !FEngineBuildSettings::IsInternalBuild();
	if( bRemovePersonalData )
	{
		FPrimaryCrashProperties::Get()->UserName = TEXT( "" );
		// For now remove the command line completely, to hide the potential personal data. Need to revisit it later.
		FPrimaryCrashProperties::Get()->CommandLine = TEXT( "CommandLineRemoved" );
	}

	// Save updated properties, including removed all PII if bAllowToBeContacted is set to false.
	FPrimaryCrashProperties::Get()->Save();

	// Remove it later, in the next iteration.
	// Find .xml file
	FString XmlFilename;
	if (!FindFirstReportFileWithExtension(XmlFilename, TEXT(".xml")))
	{
		return false;
	}

	FString XmlFilePath = GetReportDirectory() / XmlFilename;
	// FXmlFile's constructor loads the file to memory, closes the file and parses the data
	FXmlFile XmlFile(XmlFilePath);
	FXmlNode* DynamicSignaturesNode = XmlFile.IsValid() ?
		XmlFile.GetRootNode()->FindChildNode(TEXT("DynamicSignatures")) :
		nullptr;

	if (!DynamicSignaturesNode)
	{
		return false;
	}

	if (bRemovePersonalData)
	{
		FXmlNode* ProblemNode = XmlFile.GetRootNode()->FindChildNode( TEXT( "ProblemSignatures" ) );
		if (ProblemNode)
		{
			FXmlNode* Parameter8Node = ProblemNode->FindChildNode( TEXT( "Parameter8" ) );
			if (Parameter8Node)
			{
				// Replace user name in assert message, command line etc.
				FString Content = Parameter8Node->GetContent();
				Content = Content.Replace( *UserName1, Anonymous );
				Content = Content.Replace( *UserName2, Anonymous );

				// Remove the command line. Command line is between first and second !
				TArray<FString> ParsedParameters8;
				Content.ParseIntoArray( ParsedParameters8, TEXT( "!" ), false );
				if (ParsedParameters8.Num() > 1)
				{
					ParsedParameters8[1] = TEXT( "CommandLineRemoved" );
				}

				Content = FString::Join( ParsedParameters8, TEXT( "!" ) );

				Parameter8Node->SetContent( Content );
			}
			FXmlNode* Parameter9Node = ProblemNode->FindChildNode( TEXT( "Parameter9" ) );
			if (Parameter9Node)
			{
				// Replace user name in assert message, command line etc.
				FString Content = Parameter9Node->GetContent();
				Content = Content.Replace( *UserName1, Anonymous );
				Content = Content.Replace( *UserName2, Anonymous );
				Parameter9Node->SetContent( Content );
			}
		}
	}

	// Add or update the user comment.
	FXmlNode* Parameter3Node = DynamicSignaturesNode->FindChildNode(TEXT("Parameter3"));
	if( Parameter3Node )
	{
		Parameter3Node->SetContent(UserComment.ToString());
	}
	else
	{
		DynamicSignaturesNode->AppendChildNode(TEXT("Parameter3"), UserComment.ToString());
	}

	// @see FCrashDescription::UpdateIDs
	const FString EpicLoginAndUserNameIDs = FString::Printf( TEXT( "!LoginId:%s!EpicAccountId:%s!Name:%s" ), *FPrimaryCrashProperties::Get()->LoginId.AsString(), *FPrimaryCrashProperties::Get()->EpicAccountId.AsString(), *FPrimaryCrashProperties::Get()->UserName.AsString() );

	// Add or update a user ID.
	FXmlNode* Parameter4Node = DynamicSignaturesNode->FindChildNode(TEXT("Parameter4"));
	if( Parameter4Node )
	{
		Parameter4Node->SetContent(EpicLoginAndUserNameIDs);
	}
	else
	{
		DynamicSignaturesNode->AppendChildNode(TEXT("Parameter4"), EpicLoginAndUserNameIDs);
	}

	// Add or update bAllowToBeContacted
	const FString AllowToBeContacted = bAllowToBeContacted ? TEXT("true") : TEXT("false");
	FXmlNode* AllowToBeContactedNode = DynamicSignaturesNode->FindChildNode(TEXT("bAllowToBeContacted"));
	if( AllowToBeContactedNode )
	{
		AllowToBeContactedNode->SetContent(AllowToBeContacted);
	}
	else
	{
		DynamicSignaturesNode->AppendChildNode(TEXT("bAllowToBeContacted"), AllowToBeContacted);
	}

	// Re-save over the top
	return XmlFile.Save(XmlFilePath);
}

void FGenericErrorReport::SetPrimaryCrashProperties( FPrimaryCrashProperties& out_PrimaryCrashProperties )
{
#if WITH_CRASHREPORTER
	FCrashDebugHelperModule& CrashHelperModule = FModuleManager::LoadModuleChecked<FCrashDebugHelperModule>( FName( "CrashDebugHelper" ) );
	ICrashDebugHelper* Helper = CrashHelperModule.Get();
	if (Helper && bValidCallstack)
	{
		TArray<FString> CallStack = Helper->CrashInfo.Exception.CallStackString;
		TArray<FString> PCallStack = Helper->CrashInfo.Exception.PortableCallStackString;

		// Get the callstack and remove any frames that we don't care about
		int64 NumMinidumpFramesToIgnore = out_PrimaryCrashProperties.NumMinidumpFramesToIgnore;
		if (NumMinidumpFramesToIgnore > 0)
		{
			CallStack.RemoveAt(0, FMath::Min(CallStack.Num(), (int32)NumMinidumpFramesToIgnore));
			PCallStack.RemoveAt(0, FMath::Min(PCallStack.Num(), (int32)NumMinidumpFramesToIgnore));
		}

		if (CallStack.Num() > 0)
		{
			out_PrimaryCrashProperties.CallStack = CallStack;
		}
		if (PCallStack.Num() > 0)
		{
			out_PrimaryCrashProperties.PCallStack = PCallStack;
			
			FString StringToHash = out_PrimaryCrashProperties.PCallStack.AsString();
			FSHAHash PCallStackHash;
			FSHA1::HashBuffer(*StringToHash, StringToHash.Len() * sizeof(FString::ElementType), PCallStackHash.Hash);
			out_PrimaryCrashProperties.PCallStackHashProperty = PCallStackHash.ToString();
		}
		
		out_PrimaryCrashProperties.Threads = Helper->CrashInfo.Threads;
		out_PrimaryCrashProperties.Modules = Helper->CrashInfo.ModuleNames;
		out_PrimaryCrashProperties.SourceContext = Helper->CrashInfo.SourceContext;

		// If error message is empty, it means general crash like accessing invalid memory ptr.
		if (out_PrimaryCrashProperties.ErrorMessage.AsString().Len() == 0)
		{
			out_PrimaryCrashProperties.ErrorMessage = Helper->CrashInfo.Exception.ExceptionString;
		}

		out_PrimaryCrashProperties.Save();
	}
#endif
}

void FGenericErrorReport::SetCrashReportClientVersion(const FString& InVersion)
{
	FPrimaryCrashProperties::Get()->CrashReportClientVersion = InVersion;
	FPrimaryCrashProperties::Get()->Save();
}

TArray<FString> FGenericErrorReport::GetFilesToUpload() const
{
	TArray<FString> FilesToUpload;

	for (const auto& Filename: ReportFilenames)
	{
		FilesToUpload.Push(ReportDirectory / Filename);
	}
	return FilesToUpload;
}

TArray<FString> FGenericErrorReport::GetOptionalAttachmentsToUpload() const
{
	TArray<FString> FilesToUpload;

	for (const auto& Filename : OptionalAttachmentsFilenames)
	{
		FilesToUpload.Push(OptionalAttachmentsDirectory / Filename);
	}
	return FilesToUpload;
}

void FGenericErrorReport::DeleteFiles()
{
	for (const auto& Filename: ReportFilenames)
	{
		IFileManager::Get().Delete(*(ReportDirectory / Filename));
	}
	IFileManager::Get().DeleteDirectory(*ReportDirectory);
}

bool FGenericErrorReport::LoadWindowsReportXmlFile( FString& OutString ) const
{
	// Find .xml file
	FString XmlFilename;
	if (!FindFirstReportFileWithExtension(XmlFilename, TEXT(".xml")))
	{
		return false;
	}

	return FFileHelper::LoadFileToString( OutString, *(ReportDirectory / XmlFilename) );
}

bool FGenericErrorReport::TryReadDiagnosticsFile()
{
	FString FileContent;
	if (!FFileHelper::LoadFileToString(FileContent, *(ReportDirectory / FCrashReportCoreConfig::Get().GetDiagnosticsFilename())))
	{
		// No diagnostics file
		return false;
	}

	FString Exception;
	TArray<FString> Callstack;

	static const TCHAR CallStackStartKey[] = TEXT("<CALLSTACK START>");
	static const TCHAR CallStackEndKey[] = TEXT("<CALLSTACK END>");
	static const TCHAR SourceContextStartKey[] = TEXT("<SOURCE START>");
	static const TCHAR SourceContextEndKey[] = TEXT("<SOURCE END>");
	static const TCHAR ExceptionLineStart[] = TEXT("Exception was ");
	auto ReportSection = EReportSection::Other;
	const TCHAR* Stream = *FileContent;
	FString Line;
	while (FParse::Line(&Stream, Line, true /* don't treat |s as new-lines! */))
	{
		switch (ReportSection)
		{

		case EReportSection::CallStack:
			if (Line.StartsWith(CallStackEndKey))
			{
				ReportSection = EReportSection::Other;
			}
			else
			{
				Callstack.Push(Line);
			}
			break;
		case EReportSection::SourceContext:
			if (Line.StartsWith(SourceContextEndKey))
			{
				ReportSection = EReportSection::Other;
			}
			else
			{
				// Not doing anything with this at the moment
			}
			break;
		case EReportSection::Other:
			if (Line.StartsWith(CallStackStartKey))
			{
				ReportSection = EReportSection::CallStack;
			}
			else if (Line.StartsWith(SourceContextStartKey))
			{
				ReportSection = EReportSection::SourceContext;
			}
			else if (Line.StartsWith(ExceptionLineStart))
			{
				// Not subtracting 1 from the array count so it gobbles the initial quote
				Exception = Line.RightChop(UE_ARRAY_COUNT(ExceptionLineStart)).LeftChop(1);
			}
			break;
		}
	}

	// Update properties for the crash.
	FPrimaryCrashProperties::Get()->CallStack = Callstack;
	// If error message is empty, it means general crash like accessing invalid memory ptr.
	if (FPrimaryCrashProperties::Get()->ErrorMessage.AsString().Len() == 0)
	{
		FPrimaryCrashProperties::Get()->ErrorMessage = Exception;
	}

	return true;
}

bool FGenericErrorReport::FindFirstReportFileWithExtension( FString& OutFilename, const TCHAR* Extension ) const
{
	for (const auto& Filename: ReportFilenames)
	{
		if (Filename.EndsWith(Extension))
		{
			OutFilename = Filename;
			return true;
		}
	}
	return false;
}

FString FGenericErrorReport::FindCrashedAppName() const
{
	FString AppPath = FindCrashedAppPath();
	if (!AppPath.IsEmpty())
	{
		return FPaths::GetCleanFilename(AppPath);
	}

	return AppPath;
}

FString FGenericErrorReport::FindCrashedAppPath() const
{
	UE_LOGF( LogTemp, Warning, "FGenericErrorReport::FindCrashedAppPath not implemented on this platform" );
	return FString( "GenericAppPath" );
}

