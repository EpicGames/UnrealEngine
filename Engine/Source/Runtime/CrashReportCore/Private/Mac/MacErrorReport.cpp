// Copyright Epic Games, Inc. All Rights Reserved.

#include "Mac/MacErrorReport.h"
#include "../CrashReportUtil.h"
#include "Misc/FileHelper.h"
#include "HAL/PlatformFileManager.h"
#include "Modules/ModuleManager.h"
#include "CrashReportCoreConfig.h"

#if WITH_CRASHREPORTER
#include "Mac/MacSystemIncludes.h"
#include "CrashDebugHelper.h"
#include "CrashDebugHelperModule.h"
#endif

#define LOCTEXT_NAMESPACE "CrashReportClient"

namespace
{
#if WITH_CRASHREPORTER
	/** Pointer to dynamically loaded crash diagnosis module */
	FCrashDebugHelperModule* CrashHelperModule;
#endif
}

FMacErrorReport::FMacErrorReport(const FString& Directory)
	: FGenericErrorReport(Directory)
{
}

void FMacErrorReport::Init()
{
#if WITH_CRASHREPORTER
	CrashHelperModule = &FModuleManager::LoadModuleChecked<FCrashDebugHelperModule>(FName("CrashDebugHelper"));
#endif
}

void FMacErrorReport::ShutDown()
{
#if WITH_CRASHREPORTER
	CrashHelperModule->ShutdownModule();
#endif
}

FString FMacErrorReport::FindCrashedAppPath() const
{
#if WITH_CRASHREPORTER
	TArray<uint8> Data;
	if(FFileHelper::LoadFileToArray(Data, *(ReportDirectory / TEXT("Report.wer"))))
	{
		CFStringRef CFString = CFStringCreateWithBytes(NULL, Data.GetData(), Data.Num(), kCFStringEncodingUTF16LE, true);
		FString FileData((NSString*)CFString);
		CFRelease(CFString);
		
		static const TCHAR AppPathLineStart[] = TEXT("AppPath=");
		static const int AppPathIdLength = UE_ARRAY_COUNT(AppPathLineStart) - 1;
		int32 AppPathStart = FileData.Find(AppPathLineStart);
		if(AppPathStart >= 0)
		{
			FString PathData = FileData.Mid(AppPathStart + AppPathIdLength);
			int32 LineEnd = -1;
			if(PathData.FindChar( TCHAR('\r'), LineEnd ))
			{
				PathData.LeftInline(LineEnd, EAllowShrinking::No);
			}
			if(PathData.FindChar( TCHAR('\n'), LineEnd ))
			{
				PathData.LeftInline(LineEnd, EAllowShrinking::No);
			}
			return PathData;
		}
	}
	else
	{
		UE_LOGF(LogStreaming, Error,	"Failed to read file '%ls' error.",*(ReportDirectory / TEXT("Report.wer")));
	}
#endif
	return "";
}

void FMacErrorReport::FindMostRecentErrorReports(TArray<FString>& ErrorReportPaths, const FTimespan& MaxCrashReportAge)
{
#if WITH_CRASHREPORTER
	IPlatformFile& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	FDateTime MinCreationTime = FDateTime::UtcNow() - MaxCrashReportAge;
	auto ReportFinder = MakeDirectoryVisitor([&](const TCHAR* FilenameOrDirectory, bool bIsDirectory)
	{
		if (bIsDirectory)
		{
			auto TimeStamp = PlatformFile.GetTimeStamp(FilenameOrDirectory);
			if (TimeStamp > MinCreationTime)
			{
				ErrorReportPaths.Add(FilenameOrDirectory);
			}
		}
		return true;
	});

	FString AllReportsDirectory = FPaths::GameAgnosticSavedDir() / TEXT("Crashes");

	PlatformFile.IterateDirectory(
		*AllReportsDirectory,
		ReportFinder);

	ErrorReportPaths.Sort([&](const FString& L, const FString& R)
	{
		auto TimeStampL = PlatformFile.GetTimeStamp(*L);
		auto TimeStampR = PlatformFile.GetTimeStamp(*R);

		return TimeStampL > TimeStampR;
	});
#endif
}

FText FMacErrorReport::DiagnoseReport() const
{
#if WITH_CRASHREPORTER
	// Should check if there are local PDBs before doing anything
	ICrashDebugHelper* CrashDebugHelper = CrashHelperModule ? CrashHelperModule->Get() : nullptr;
	if (!CrashDebugHelper)
	{
		// Not localized: should never be seen
		return FText::FromString(TEXT("Failed to load CrashDebugHelper."));
	}
	
	FString DumpFilename;
	if (!FindFirstReportFileWithExtension(DumpFilename, TEXT(".dmp")))
	{
		if (!FindFirstReportFileWithExtension(DumpFilename, TEXT(".mdmp")))
		{
			return FText::FromString("No minidump found for this crash.");
		}
	}
	
	FCrashDebugInfo DebugInfo;
	if (!CrashDebugHelper->ParseCrashDump(ReportDirectory / DumpFilename, DebugInfo))
	{
		return FText::FromString("No minidump found for this crash.");
	}
	
	if ( !CrashDebugHelper->CreateMinidumpDiagnosticReport(ReportDirectory / DumpFilename) )
	{
		return LOCTEXT("NoDebuggingSymbols", "You do not have any debugging symbols required to display the callstack for this crash.");
	}
	else
	{
		FString CrashDump;
		FString DiagnosticsPath = ReportDirectory / FCrashReportCoreConfig::Get().GetDiagnosticsFilename();
		CrashDebugHelper->CrashInfo.GenerateReport( DiagnosticsPath );
		if ( FFileHelper::LoadFileToString( CrashDump, *(ReportDirectory / FCrashReportCoreConfig::Get().GetDiagnosticsFilename() ) ) )
		{
			return FText::FromString(CrashDump);
		}
		else
		{
			return FText::FromString("Failed to create diagnosis information.");
		}
	}
#else
	return FText::FromString(TEXT("Failed to load CrashDebugHelper."));
#endif
}

#undef LOCTEXT_NAMESPACE
