// Copyright Epic Games, Inc. All Rights Reserved.

#include "PendingReports.h"
#include "CrashReportCoreModule.h"
#include "HAL/PlatformFileManager.h"
#include "Logging/LogMacros.h"
#include "HAL/FileManager.h"
#include "Misc/Paths.h"
#include "Templates/UniquePtr.h"

#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"
#include "Serialization/JsonSerializer.h"

namespace
{
	typedef TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>> FPrettyJsonWriter;
	typedef TArray<TSharedPtr<FJsonValue>> FJsonReportArray;

	const TCHAR ReportsArrayFieldName[] = TEXT("crash-reports");
}

FPendingReports::FPendingReports()
{
	Load();
}

void FPendingReports::Add( const FString& Path )
{
	FString NormalizedPath = Path;
	FPaths::NormalizeDirectoryName(NormalizedPath);
	Reports.AddUnique(NormalizedPath);
}

void FPendingReports::Forget(const FString& ReportDirectoryName)
{
	auto Index = Reports.IndexOfByPredicate([&](const FString& Path) {
		return FPaths::GetCleanFilename(Path) == ReportDirectoryName;
	});

	if (Index != INDEX_NONE)
	{
		Reports.RemoveAt(Index, 1 /* single item */, EAllowShrinking::No);
	}
}

void FPendingReports::Save() const
{
	auto PendingReportsPath = GetPendingReportsJsonFilepath();

	// Ensure directory structure exists
	FPlatformFileManager::Get().GetPlatformFile().CreateDirectoryTree(*FPaths::GetPath(PendingReportsPath));
	
	FJsonReportArray JsonReports;
	for (auto Path : Reports)
	{
		JsonReports.Push(MakeShareable(new FJsonValueString(Path)));
	}
	
	TSharedRef<FJsonObject> JsonRootObject = MakeShareable(new FJsonObject);
	JsonRootObject->SetArrayField(ReportsArrayFieldName, JsonReports);

	TUniquePtr<FArchive> FileWriter(IFileManager::Get().CreateFileWriter(*PendingReportsPath));
	if (!FileWriter)
	{
		UE_LOGF(CrashReportCoreLog, Warning, "Failed to save pending reports JSON file");
		return;
	}
	FJsonSerializer::Serialize(JsonRootObject, FPrettyJsonWriter::Create(FileWriter.Get()));
}

const TArray<FString>& FPendingReports::GetReportDirectories() const
{
	return Reports;
}

void FPendingReports::Load()
{
	TUniquePtr<FArchive> FileReader(IFileManager::Get().CreateFileReader(*GetPendingReportsJsonFilepath()));
	if (!FileReader)
	{
		return;
	}
	
	TSharedPtr<FJsonObject> JsonRootObject;
	if (!FJsonSerializer::Deserialize(TJsonReader<>::Create(FileReader.Get()), JsonRootObject))
	{
		return;
	}

	// Array will be empty if there's a type mismatch
	auto ReportArray = JsonRootObject->GetArrayField(ReportsArrayFieldName);

	for (auto PathValue: ReportArray)
	{
		auto Path = PathValue->AsString();
		if (!Path.IsEmpty())
		{
			Reports.Add(Path);
		}
	}
}

FString FPendingReports::GetPendingReportsJsonFilepath()
{
	return FPaths::GameAgnosticSavedDir() / TEXT("crash-reports/pending-reports.json");
}

