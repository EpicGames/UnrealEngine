// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/UnrealString.h"

/**
 * Manager for reports that weren't able to be sent on previous runs of the tool
 */
class FPendingReports
{
public:
	/**
	 * Load the list of reports
	 */
	FPendingReports();

	/**
	 * Add a pending report directory
	 * @Path Full path to the directory containing the report
	 */
	void Add(const FString& Path);

	/**
	 * Remove a pending report directory if present
	 * @ReportDirectoryName Leaf name of report directory to remove
	 */
	void Forget(const FString& ReportDirectoryName);

	/**
	 * Clear out the list of reports
	 */
	void Clear()
	{
		Reports.Reset();
	}

	/**
	 * Save out the list of reports to the user's application settings folder
	 */
	void Save() const;

	/**
	 * Access to report directories
	 * @return Full paths to all pending report directories
	 */
	const TArray<FString>& GetReportDirectories() const;

private:
	/**
	 * Load the list of reports from the user's application settings folder
	 */
	void Load();

	/**
	 * Get the application settings location of the pending reports file
	 * @return Full path to the file
	 */
	static FString GetPendingReportsJsonFilepath();

	/** Full paths to reports not yet submitted */
	TArray<FString> Reports;
};
