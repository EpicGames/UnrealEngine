// Copyright Epic Games, Inc. All Rights Reserved.

#include "Windows/WindowsErrorReport.h"
#include "CrashReportUtil.h"
#include "CrashDescription.h"
#include "Misc/FileHelper.h"
#include "Modules/ModuleManager.h"
#include "Internationalization/Text.h"
#include "Internationalization/Internationalization.h"
#include "HAL/PlatformFileManager.h"

#include "Windows/AllowWindowsPlatformTypes.h"
#include <ShlObj.h>
#include "Windows/HideWindowsPlatformTypes.h"

#if WITH_CRASHREPORTER
#include "CrashDebugHelperModule.h"
#include "CrashDebugHelper.h"
#endif

#define LOCTEXT_NAMESPACE "CrashReportClient"

namespace
{
#if WITH_CRASHREPORTER
	/** Pointer to dynamically loaded crash diagnosis module */
	FCrashDebugHelperModule* CrashHelperModule;
#endif
}

/** Helper class used to parse specified string value based on the marker. */
struct FWindowsReportParser
{
	static FString Find( const FString& ReportDirectory, const TCHAR* Marker )
	{
		FString Result;

		TArray<uint8> FileData;
		FFileHelper::LoadFileToArray( FileData, *(ReportDirectory / TEXT( "Report.wer" )) );
		FileData.Add( 0 );
		FileData.Add( 0 );

		const FString FileAsString = reinterpret_cast<TCHAR*>(FileData.GetData());

		TArray<FString> String;
		FileAsString.ParseIntoArray( String, TEXT( "\r\n" ), true );

		for( const auto& StringLine : String )
		{
			if( StringLine.Contains( Marker ) )
			{
				TArray<FString> SeparatedParameters;
				StringLine.ParseIntoArray( SeparatedParameters, Marker, true );

				Result = SeparatedParameters[SeparatedParameters.Num()-1];
				break;
			}
		}

		return Result;
	}
};

FWindowsErrorReport::FWindowsErrorReport(const FString& Directory)
	: FGenericErrorReport(Directory)
{
}

void FWindowsErrorReport::Init()
{
#if WITH_CRASHREPORTER
	CrashHelperModule = &FModuleManager::LoadModuleChecked<FCrashDebugHelperModule>(FName("CrashDebugHelper"));
#endif
}

void FWindowsErrorReport::ShutDown()
{
#if WITH_CRASHREPORTER
	CrashHelperModule->ShutdownModule();
#endif
}

FString FWindowsErrorReport::FindCrashedAppPath() const
{
	FString AppPath = FPaths::Combine(FPrimaryCrashProperties::Get()->BaseDir, FPrimaryCrashProperties::Get()->ExecutableName);
	AppPath += TEXT(".exe");
	return AppPath;
}

FText FWindowsErrorReport::DiagnoseReport() const
{
	// Mark the callstack as invalid.
	bValidCallstack = false;

#if WITH_CRASHREPORTER
	// Should check if there are local PDBs before doing anything
	auto CrashDebugHelper = CrashHelperModule ? CrashHelperModule->GetNew() : nullptr;
	if (!CrashDebugHelper)
	{
		// Not localized: should never be seen
		return FText::FromString(TEXT("Failed to load CrashDebugHelper."));
	}

	FString DumpFilename;
	if (!FindFirstReportFileWithExtension(DumpFilename, TEXT(".dmp")))
	{
		if (!FindFirstReportFileWithExtension(DumpFilename, TEXT(".mdmp")))
		{
			return LOCTEXT("MinidumpNotFound", "No minidump found for this crash.");
		}
	}

	if (!CrashDebugHelper->CreateMinidumpDiagnosticReport(ReportDirectory / DumpFilename))
	{
		return LOCTEXT("NoDebuggingSymbols", "You do not have any debugging symbols required to display the callstack for this crash.");
	}

	// No longer required, only for backward compatibility, mark the callstack as valid.
	bValidCallstack = true;
#endif
	return FText();
}

static bool TryGetDirectoryCreationTimeUtc(const FString& InDirectoryName, FDateTime& OutCreationTime)
{
	FString DirectoryName(InDirectoryName);
	FPaths::MakePlatformFilename(DirectoryName);

	WIN32_FILE_ATTRIBUTE_DATA Info;
	if (!GetFileAttributesExW(*DirectoryName, GetFileExInfoStandard, &Info))
	{
		OutCreationTime = FDateTime();
		return false;
	}

	SYSTEMTIME SysTime;
	if (!FileTimeToSystemTime(&Info.ftCreationTime, &SysTime))
	{
		OutCreationTime = FDateTime();
		return false;
	}

	OutCreationTime = FDateTime(SysTime.wYear, SysTime.wMonth, SysTime.wDay, SysTime.wHour, SysTime.wMinute, SysTime.wSecond);
	return true;
}

void FWindowsErrorReport::FindMostRecentErrorReports(TArray<FString>& ErrorReportPaths, const FTimespan& MaxCrashReportAge)
{
#if WITH_CRASHREPORTER
	auto& PlatformFile = FPlatformFileManager::Get().GetPlatformFile();

	FDateTime MinCreationTime = FDateTime::UtcNow() - MaxCrashReportAge;
	auto ReportFinder = MakeDirectoryVisitor([&](const TCHAR* FilenameOrDirectory, bool bIsDirectory) 
	{
		if (bIsDirectory)
		{
			FDateTime CreationTime;
			if (TryGetDirectoryCreationTimeUtc(FilenameOrDirectory, CreationTime) && CreationTime > MinCreationTime && FCString::Strstr(FilenameOrDirectory, TEXT("UE-")))
			{
				ErrorReportPaths.Add(FilenameOrDirectory);
			}
		}
		return true;
	});

	{
		TCHAR LocalAppDataPath[MAX_PATH];
		SHGetFolderPath(0, CSIDL_LOCAL_APPDATA, NULL, 0, LocalAppDataPath);
		PlatformFile.IterateDirectory(*(FString(LocalAppDataPath) / TEXT("Microsoft/Windows/WER/ReportQueue")), ReportFinder);
	}

	if (ErrorReportPaths.Num() == 0)
	{
		TCHAR LocalAppDataPath[MAX_PATH];
		SHGetFolderPath( 0, CSIDL_COMMON_APPDATA, NULL, 0, LocalAppDataPath );
		PlatformFile.IterateDirectory( *(FString( LocalAppDataPath ) / TEXT( "Microsoft/Windows/WER/ReportQueue" )), ReportFinder );
	}

	ErrorReportPaths.Sort([](const FString& L, const FString& R)
	{
		FDateTime CreationTimeL;
		TryGetDirectoryCreationTimeUtc(L, CreationTimeL);
		FDateTime CreationTimeR;
		TryGetDirectoryCreationTimeUtc(R, CreationTimeR);

		return CreationTimeL > CreationTimeR;
	});
#endif
}

#undef LOCTEXT_NAMESPACE
