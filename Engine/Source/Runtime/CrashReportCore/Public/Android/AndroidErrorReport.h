// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"
#include "GenericErrorReport.h"

struct FTimespan;

/**
 * Helper that works with Android Error Reports
 */
class FAndroidErrorReport : public FGenericErrorReport
{
public:
	/**
	 * Default constructor: creates a report with no files
	 */
	FAndroidErrorReport()
	{
	}

	/**
	 * Discover all files in the crash report directory
	 * @param Directory Full path to directory containing the report
	 */
	CRASHREPORTCORE_API explicit FAndroidErrorReport(const FString& Directory);

	/**
	 * Load helper modules
	 */
	static void Init();

	/**
	 * Unload helper modules
	 */
	static void ShutDown();

	/**
	 */
	CRASHREPORTCORE_API FText DiagnoseReport() const;

	/**
	 * Do nothing - shouldn't be called on Android
	 * @return Empty string
	 */
	static void FindMostRecentErrorReports(TArray<FString>& ErrorReportPaths, const FTimespan& MaxCrashReportAge)
	{
		// The report folder is currently always sent on the command-line on Android
	}

	/**
	 * Get the full path of the crashed app from the report
	 */
	FString FindCrashedAppPath() const
	{
		FString AppPath = FPaths::Combine(FPrimaryCrashProperties::Get()->BaseDir, FPrimaryCrashProperties::Get()->ExecutableName);
		return AppPath;
	}
protected:

	class FMergeInfo
	{
		FMergeInfo();
	public:
		enum class EMergeType { Threads, PlatformProperties };
		FMergeInfo(FString MergeNameIn, EMergeType MergeTypeIn) : MergeName(MergeNameIn), MergeType(MergeTypeIn)
		{}

		EMergeType MergeType;
		FString MergeName;
	};
	/**
	 * Filenames and merge info type of mergeable source files.
	 */
	TArray<FMergeInfo> MergeSources;
};
