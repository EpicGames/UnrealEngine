// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformCrashContext.h"
#include "HAL/Platform.h"
#include "Misc/CString.h"
#include "Misc/EngineVersion.h"
#include "Templates/Function.h"
#include "XmlFile.h"
#include "XmlNode.h"

#if WITH_CRASHREPORTER
#include "CrashDebugHelper.h"
#endif

enum class ECrashDescVersions : int32;
enum class ECrashDumpMode : int32;
class FXmlNode;
struct FAnalyticsEventAttribute;
struct FPrimaryCrashProperties;

/** PrimaryCrashProperties. Extracted from: FGenericCrashContext::SerializeContentToBuffer */
/*
	"CrashVersion"
	"ProcessId"
	"CrashGUID"
	"IsInternalBuild"
	"IsPerforceBuild"
	"IsSourceDistribution"
	"IsEnsure"
	"SecondsSinceStart"
	"GameName"
	"ExecutableName"
	"BuildConfiguration"
	"PlatformName"
	"PlatformNameIni"
	"PlatformFullName"
	"EngineMode"
	"EngineModeEx"
	"EngineVersion"
	"BuildVersion"
	"CommandLine"
	"LanguageLCID"
	"AppDefaultLocale"
	"IsUERelease"
	"UserName"
	"BaseDir"
	"RootDir"
	"MachineId"
	"LoginId"
	"EpicAccountId"
	"CallStack"
	"SourceContext"
	"UserDescription"
	"UserActivityHint"
	"ErrorMessage"
	"CrashDumpMode"
	"CrashReporterMessage"
	"Misc.NumberOfCores"
	"Misc.NumberOfCoresIncludingHyperthreads"
	"Misc.Is64bitOperatingSystem"
	"Misc.CPUVendor"
	"Misc.CPUBrand"
	"Misc.PrimaryGPUBrand"
	"Misc.OSVersionMajor"
	"Misc.OSVersionMinor"
	"Misc.AppDiskTotalNumberOfBytes"
	"Misc.AppDiskNumberOfFreeBytes"
	"MemoryStats.TotalPhysical"
	"MemoryStats.TotalVirtual"
	"MemoryStats.PageSize"
	"MemoryStats.TotalPhysicalGB"
	"MemoryStats.AvailablePhysical"
	"MemoryStats.AvailableVirtual"
	"MemoryStats.UsedPhysical"
	"MemoryStats.PeakUsedPhysical"
	"MemoryStats.UsedVirtual"
	"MemoryStats.PeakUsedVirtual"
	"MemoryStats.bIsOOM"
	"MemoryStats.OOMAllocationSize"
	"MemoryStats.OOMAllocationAlignment"
	"TimeofCrash"
	"bAllowToBeContacted"
 */

inline void LexFromString( ECrashDescVersions& OutValue, const TCHAR* Buffer )
{
	OutValue = (ECrashDescVersions)FCString::Atoi( Buffer );
}

inline void LexFromString( ECrashDumpMode& OutValue, const TCHAR* Buffer )
{
	OutValue = (ECrashDumpMode)FCString::Atoi( Buffer );
}

inline void LexFromString( FEngineVersion& OutValue, const TCHAR* Buffer )
{
	FEngineVersion::Parse( Buffer, OutValue );
}

/** Simple crash property. Only for string values. */
struct FCrashProperty
{
	friend struct FPrimaryCrashProperties;

protected:
	/** Initialization constructor. */
	FCrashProperty( const FString& InMainCategory, const FString& InSecondCategory, FPrimaryCrashProperties* InOwner );

public:
	/** Assignment operator for string. */
	FCrashProperty& operator=(const FString& NewValue);

	/** Assignment operator for TCHAR*. */
	FCrashProperty& operator=(const TCHAR* NewValue);

	/** Assignment operator for arrays. */
	FCrashProperty& operator=(const TArray<FString>& NewValue);

	/** Assignment operator for bool. */
	FCrashProperty& operator=(const bool NewValue);

	/** Assignment operator for int64. */
	FCrashProperty& operator=(const int64 NewValue);

	/** Getter for string, default. */
	CRASHREPORTCORE_API const FString& AsString() const;

	/** Getter for bool. */
	CRASHREPORTCORE_API bool AsBool() const;

	/** Getter for int64. */
	CRASHREPORTCORE_API int64 AsInt64() const;

protected:
	/** Owner of the property. */
	FPrimaryCrashProperties* Owner;

	/** Cached value of the property. */
	mutable FString CachedValue;

	/** Main category in the crash context. */
	FString MainCategory;

	/** Second category in the crash context. */
	FString SecondCategory;

	mutable bool bSet;
};

struct FCrashPropertyXmlNode
{
	friend struct FPrimaryCrashProperties;
	/** Assignment operator for . */
	FCrashPropertyXmlNode& operator=(const FXmlNode* Node);
	
#if WITH_CRASHREPORTER
	/** Assignment operator for converting ThreadsInfo into xml nodes*/
	FCrashPropertyXmlNode& operator=(const TArray<FCrashThreadInfo>& ThreadsInfo);
#endif
	
protected:
	/** Initialization constructor. */
	FCrashPropertyXmlNode(const FString& InMainCategory, const FString& InSecondCategory, FPrimaryCrashProperties* InOwner);

protected:
	/** Owner of the property. */
	FPrimaryCrashProperties* Owner;

	/** Main category in the crash context. */
	FString MainCategory;

	/** Second category in the crash context. */
	FString SecondCategory;

	mutable bool bSet;
};

/** Primary crash properties required by the crash report system. */
struct FPrimaryCrashProperties
{
	friend struct FCrashProperty;
	friend struct FCrashPropertyXmlNode;

	/** Version. */
	ECrashDescVersions CrashVersion;

	/** Crash dump mode. */
	ECrashDumpMode CrashDumpMode;

	/** An unique report name that this crash belongs to. Folder name. */
	FString CrashGUID;

	/**
	 * The name of the game that crashed. (AppID)
	 * @GameName	varchar(64)
	 * 
	 * FApp::GetProjectName()
	 */
	FString GameName;

	/**
	* The name of the exe that crashed. (AppID)
	* @GameName	varchar(64)
	*/
	FString ExecutableName;

	/**
	 * The mode the game was in e.g. editor.
	 * @EngineMode	varchar(64)
	 * 
	 * FPlatformMisc::GetEngineMode()
	 */
	FString EngineMode;

	/**
	 * Deployment (also known as "EpicApp"), e.g. DevPlaytest, PublicTest, Live
	 * @DeploymentName varchar(64)
	 */
	FString DeploymentName;

	/**
	 * EngineModeEx e.g. Unset, Dirty, Vanilla
	 * @DeploymentName varchar(64)
	 */
	FCrashProperty EngineModeEx;

	/**
	 * The platform that crashed e.g. Win64.
	 * @PlatformName	varchar(64)	
	 * 
	 * Last path of the directory
	 */
	FCrashProperty PlatformFullName;

	/** 
	 * Encoded engine version. (AppVersion)
	 * E.g. 4.3.0.0-2215663+UE-Releases+4.3
	 * BuildVersion-BuiltFromCL-BranchName
	 * @EngineVersion	varchar(64)	
	 * 
	 * FEngineVersion::Current().ToString()
	 * ENGINE_VERSION_STRING
	 */
	FEngineVersion EngineVersion;

	/**
	 * The command line of the application that crashed.
	 * @CommandLine varchar(512)
	 * 
	 * FCommandLine::Get() 
	 */
	FCrashProperty CommandLine;

	/**
	 * The base directory where the app was running.
	 * @BaseDir varchar(512)
	 * 
	 * FPlatformProcess::BaseDir()
	 */
	FString BaseDir;

	/**
	 * The language ID the application that crashed.
	 * @LanguageExt varchar(64)
	 * 
	 * FPlatformMisc::GetDefaultLocale()
	 */
	FString AppDefaultLocale;

	/** 
	 * The name of the user that caused this crash.
	 * @UserName varchar(64)
	 * 
	 * FString( FPlatformProcess::UserName() ).Replace( TEXT( "." ), TEXT( "" ) )
	 */
	FCrashProperty UserName;

	/**
	 * The unique ID used to identify the machine the crash occurred on.
	 * @ComputerName varchar(64)
	 * 
	 * FPlatformMisc::GetLoginId()
	 */
	FCrashProperty LoginId;

	/** 
	 * The Epic account ID for the user who last used the Launcher.
	 * @EpicAccountId	varchar(64)
	 * 
	 * FPlatformMisc::GetEpicAccountId()
	 */
	FCrashProperty EpicAccountId;

	/** 
	 * The last game session id set by the application. Application specific meaning. Some might not set this.
	 * @EpicAccountId	varchar(64)
	 * 
	 */
	FCrashProperty GameSessionID;

	/*
	 * A hash representing a unique id for a portable callstack. These will be specific to the CL version of the application
	 * @PCallStackHash	varchar(160)
	 *
	 */
	FString PCallStackHash;

	/*
	 * The signal that was raised to enter the crash handler
	 */
	int32 CrashSignal;

	/**
	 * Specifies the number of stack frames in the callstack to ignore when symbolicating from a minidump.
	 */
	int32 NumMinidumpFramesToIgnore;

	/**
	 * An array of FStrings representing the callstack of the crash.
	 * @RawCallStack	varchar(MAX)
	 * 
	 */
	FCrashProperty CallStack;
	
	/**
	 * An array of FStrings representing the portable callstack of the crash.
	 *
	 */
	FCrashProperty PCallStack;
	
	/**
	 * The hash of PCallStack
	 *
	 */
	FCrashProperty PCallStackHashProperty;

	/**
	 * An array of FStrings showing the source code around the crash.
	 * @SourceContext varchar(max)
	 * 
	 */
	FCrashProperty SourceContext;

	/**
	* An array of module's name used by the game that crashed.
	*
	*/
	FCrashProperty Modules;

	/**
	 * An array of FStrings representing the user description of the crash.
	 * @Description	varchar(512)
	 * 
	 */
	FCrashProperty UserDescription;

	/**
	 * An FString representing the user activity, if known, when the error occurred.
	 * @UserActivityHint varchar(512)
	 *
	 */
	FCrashProperty UserActivityHint;

	/**
	 * The error message, can be assertion message, ensure message or message from the fatal error.
	 * @Summary	varchar(512)
	 * 
	 * GErrorMessage
	 */
	FCrashProperty ErrorMessage;

	/** Location of full crash dump. Displayed in the crash report frontend. */
	FCrashProperty FullCrashDumpLocation;

	/**
	 * The UTC time the crash occurred.
	 * @TimeOfCrash	datetime
	 * 
	 * FDateTime::UtcNow().GetTicks()
	 */
	FCrashProperty TimeOfCrash;

	/**
	 *	Whether the user allowed us to be contacted.
	 *	If true the following properties are retrieved from the system: UserName (for non-launcher build) and EpicAccountID.
	 *	Otherwise they will be empty.
	 */
	FCrashProperty bAllowToBeContacted;

	/**
	 *	Rich text string (should be localized by the crashing application) that will be displayed in the main CRC dialog
	 *  Can be empty and the CRC's default text will be shown.
	 */
	FCrashProperty CrashReporterMessage;

	/**
	 *	Platform-specific UE Core value (integer).
	 */
	FCrashProperty PlatformCallbackResult;

	/**
	 *	CRC sets this to the current version of the software.
	 */
	FCrashProperty CrashReportClientVersion;

	/**
	 * Whether this crash has a minidump file.
	 * @HasMiniDumpFile bit 
	 */
	bool bHasMiniDumpFile;

	/**
	 * Whether this crash has a log file.
	 * @HasLogFile bit 
	 */
	bool bHasLogFile;

	/** Whether this crash contains primary usable data. */
	bool bHasPrimaryData;

	/** 
	 * Copy of CommandLine that isn't anonymized so it can be used to restart the process 
	 */
	FString RestartCommandLine;

	/**
	 *	Whether the report comes from a non-fatal event such as an ensure
	 */
	bool bIsEnsure;

	/**
	 *	The type of crash being reported, e.g. Assert, Ensure, Hang
	 */
	FString CrashType;

	/**
	 *	The cpu brand of the device, e.g. Intel, iPhone6, etc.
	 */
	FCrashProperty CPUBrand;

	/**
	 *	Thread contexts, XML elements containing info specific to an active thread, e.g. callstacks
	 */
	FCrashPropertyXmlNode Threads;

	/**
	 *	Optional additional data for platform properties
	 */
	FCrashPropertyXmlNode PlatformPropertiesExtras;

	/**
	 * Whether it was an OOM or not
	 */
	bool bIsOOM;
	
	/**
	 * Whether we got a low memory warning or not
	 */
	bool bLowMemoryWarning;
	
	/**
	 * Whether we were in the background when the crash happened
	 */
	bool bInBackground;

	/**
	 * Whether we crashed during shutdown
	 */
	bool bIsRequestingExit;

	/**
	 * Width and height of the crash screenshot, if present
	 */
	uint32 CrashScreenshotWidth = 0;
	uint32 CrashScreenshotHeight = 0;

protected:
	/** Default constructor. */
	CRASHREPORTCORE_API FPrimaryCrashProperties();

	/** Destructor. */
	~FPrimaryCrashProperties()
	{
		delete XmlFile;
	}

public:
	/** Sets new instance as the global. */
	static void Set( FPrimaryCrashProperties* NewInstance )
	{
		Singleton = NewInstance;
	}

	/**
	 * @return global instance of the primary crash properties for the currently processed/displayed crash
	 */
	static FPrimaryCrashProperties* Get()
	{
		return Singleton;
	}

	/**
	* @return false, if there is no crash
	*/
	static bool IsValid()
	{
		return Singleton != nullptr;
	}

	/** Shutdowns the global instance. */
	static void Shutdown();

	/** Whether this crash contains callstack, error message and source context thus it means that crash is complete. */
	bool HasProcessedData() const
	{
		return CallStack.AsString().Len() > 0 && ErrorMessage.AsString().Len() > 0;
	}

	/** Updates following properties: UserName, LoginID and EpicAccountID. */
	void UpdateIDs();

	/** Sends this crash for analytics (before upload). */
	void SendPreUploadAnalytics();

	/** Sends this crash for analytics (after upload) with additional stats about the upload. */
	void SendPostUploadAnalytics(double Duration, bool bResult, int32 ResponseCode, uint32 PayloadSize, uint32 ReportCount);

	/** Saves the data. */
	void Save();

	/** Gets the FilePath of the XML File. */
	const FString& GetXmlFilepath() { return XmlFilepath; }

protected:
	/** Reads previously set XML file. Optionally give a buffer to parse from. Any changes will be stored to CrashContextFilePath. */
	void ReadXML( const FString& CrashContextFilepath, const TCHAR* Buffer = nullptr );

	/** Sets the TimeOfCrash and calls UpdateIDs if the XMLFile is valid */
	void UpdateDefaults();

	/** Sets the CrashGUID based on the report's path. */
	void SetCrashGUID( const FString& Filepath );

	/** Gets a crash property from the XML file. */
	template <typename Type>
	void GetCrashProperty( Type& out_ReadValue, const FString& MainCategory, const FString& SecondCategory ) const
	{
		if (XmlFile->IsValid())
		{
			const FXmlNode* MainNode = XmlFile->GetRootNode()->FindChildNode( MainCategory );
			if (MainNode)
			{
				const FXmlNode* CategoryNode = MainNode->FindChildNode( SecondCategory );
				if (CategoryNode)
				{
					LexFromString( out_ReadValue, *FGenericCrashContext::UnescapeXMLString( CategoryNode->GetContent() ) );
				}
			}
		}
	}

	/** Sets a crash property to a new value. */
	template <typename Type>
	void SetCrashProperty( const FString& MainCategory, const FString& SecondCategory, const Type& Value )
	{
		SetCrashProperty( MainCategory, SecondCategory, *TTypeToString<Type>::ToString( Value ) );
	}

	/** Sets a crash property to a new value. */
	void SetCrashProperty( const FString& MainCategory, const FString& SecondCategory, const FString& NewValue )
	{
		if (XmlFile->IsValid())
		{
			FXmlNode* MainNode = XmlFile->GetRootNode()->FindChildNode( MainCategory );
			if (MainNode)
			{
				FXmlNode* CategoryNode = MainNode->FindChildNode( SecondCategory );
				FString EscapedValue;
				FGenericCrashContext::AppendEscapedXMLString(EscapedValue, *NewValue);
				if (CategoryNode)
				{
					CategoryNode->SetContent( EscapedValue );
				}
				else
				{
					MainNode->AppendChildNode( SecondCategory, EscapedValue );
				}
			}
		}
	}

	/** Sets a crash property to a new value. */
	void SetCrashProperty(const FString& MainCategory, const FString& SecondCategory, const FXmlNode* NewNode)
	{
		if (XmlFile->IsValid())
		{
			FXmlNode* MainNode = XmlFile->GetRootNode()->FindChildNode(MainCategory);
			if (MainNode)
			{
				FXmlNode* CategoryNode = MainNode->FindChildNode(SecondCategory);
				const FString& Content = NewNode->GetContent();

				if(!CategoryNode) // can only add.
				{
					TFunction<void(FXmlNode*, const FXmlNode*)> AppendNode;
					AppendNode = [&AppendNode](FXmlNode* DestNode, const FXmlNode* SourceNode)
					{
						DestNode->AppendChildNode(SourceNode->GetTag(), SourceNode->GetContent());
						for (const FXmlNode *SourceChild : SourceNode->GetChildrenNodes())
						{
							const TArray<FXmlNode *>& DestChildren = DestNode->GetChildrenNodes();
							AppendNode(DestChildren.Last(), SourceChild);
						}
					};

					AppendNode(MainNode, NewNode);
				}
			}
		}
	}


	/** Encodes multi line property to be saved as single line. */
	FString EncodeArrayStringAsXMLString( const TArray<FString>& ArrayString ) const;

	CRASHREPORTCORE_API void MakeCrashEventAttributes(TArray<FAnalyticsEventAttribute>& OutCrashAttributes) const;

	/** Reader for the xml file. */
	FXmlFile* XmlFile;

	/** Cached filepath. */
	FString XmlFilepath;

	/** Global instance. */
	static FPrimaryCrashProperties* Singleton;
};

/**
 *	Describes a unified crash, should be used by all platforms.
 *	Based on FGenericCrashContext, reads all saved properties, accessed by looking into read XML. 
 *	Still lacks some of the properties, they will be added later.
 *	Must contain the same properties as ...\CrashReportServer\CrashReportCommon\CrashDescription.cs.
 *	Contains all usable information about the crash. 
 *	
 */
struct FCrashContext : public FPrimaryCrashProperties
{
	/** Initializes instance based on specified Crash Context filepath. */
	explicit FCrashContext( const FString& CrashContextFilepath, bool bIsReadOnly = false )
		: FCrashContext(CrashContextFilepath, nullptr, bIsReadOnly)
	{
	}

	/** Initializes an instance based on a in memory buffer. */
	CRASHREPORTCORE_API FCrashContext(const FString& CrashContextFilepath, const TCHAR* Buffer, bool bIsReadOnly = false);

	FCrashContext(const FCrashContext&) = delete;
	FCrashContext& operator=(const FCrashContext&) = delete;

private:

	/** Initializes the primary crash property fields from the xml file */
	void SetupPrimaryCrashProperties();
};

/** Crash context based on the Window Error Reporting WER files, obsolete, only for backward compatibility. */
struct FCrashWERContext : public FPrimaryCrashProperties
{
	/** Initializes instance based on specified WER XML filepath. */
	CRASHREPORTCORE_API explicit FCrashWERContext( const FString& WERXMLFilepath, bool bIsReadOnly = false );

	FCrashWERContext(const FCrashWERContext&) = delete;
	FCrashWERContext& operator=(const FCrashWERContext&) = delete;

	/** Initializes engine version from the separate components. */
	void InitializeEngineVersion( const FString& BuildVersion, const FString& BranchName, uint32 BuiltFromCL );
};
