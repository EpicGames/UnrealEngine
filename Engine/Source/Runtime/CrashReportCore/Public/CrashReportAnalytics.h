// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Templates/SharedPointer.h"
#include "Templates/UnrealTemplate.h"

class IAnalyticsProviderET;

/**
 * The public interface for the analytics provider singleton.
 * For Epic builds, this will point to epic's internal analytics provider.
 * For licensee builds, it will be NULL by default unless they provide their own
 * configuration.
 * 
 */
class FCrashReportAnalytics : FNoncopyable
{
public:
	/**
	 * Return the provider instance. Not valid outside of Initialize/Shutdown calls.
	 * Note: must check IsAvailable() first else this code will assert if the provider is not valid.
	 */
	static IAnalyticsProviderET& GetProvider();
	/** Helper function to determine if the provider is valid. */
	static bool IsAvailable() 
	{ 
		return Analytics.IsValid(); 
	}
	/** Called to initialize the singleton. */
	static void Initialize(const FString& EpicAccountId = FString());
	/** Called to shut down the singleton */
	static void Shutdown();

private:
	static bool bIsInitialized;
	static TSharedPtr<IAnalyticsProviderET> Analytics;
};

