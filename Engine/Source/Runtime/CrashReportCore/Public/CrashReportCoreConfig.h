// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"

// FORCE_CRASH_REPORT_UNATTENDED can be added to target.cs or UBT commandline in order to set CRASH_REPORT_UNATTENDED_ONLY
// It should not be defined in code other than when undefined to please IntelliSense.
#ifndef FORCE_CRASH_REPORT_UNATTENDED
#define FORCE_CRASH_REPORT_UNATTENDED 0
#endif

#define CRASH_REPORT_UNATTENDED_ONLY                FORCE_CRASH_REPORT_UNATTENDED

class FConfigFile;

/**
*  Holds FullCrashDump properties from the config.
*
*	FullCrashDump_0_Branch=UE
*	FullCrashDump_0_Location=\\epicgames.net\root\Builds\UE
*	FullCrashDump_1_Branch=...
*	...
*/
struct FFullCrashDumpEntry
{
	/** Initialization constructor. */
	FFullCrashDumpEntry( const FString& InBranchName, const FString& InLocation, const bool bInExactMatch )
		: BranchName( InBranchName )
		, Location( InLocation )
		, bExactMatch( bInExactMatch )
	{}


	/** Partial branch name. */
	const FString BranchName;

	/** Location where the full crash dump will be copied. Usually a network share. */
	const FString Location;

	/**
	*	Branch=UE means exact match
	*	Branch=UE* means contain match
	*/
	const bool bExactMatch;
};

/** Holds basic configuration for the crash report client. */
struct FCrashReportCoreConfig
{
	/** Accesses the singleton. */
	static FCrashReportCoreConfig& Get()
	{
		static FCrashReportCoreConfig Instance;
		return Instance;
	}

	/** Initialization constructor. */
	FCrashReportCoreConfig();

	const FString& GetVersion() const
	{
		return CrashReportClientVersion;
	}

	const FString& GetReceiverAddress() const
	{
		return CrashReportReceiverIP;
	}

	const FString& GetDataRouterURL() const
	{
		return DataRouterUrl;
	}

	const FString& GetDataRouterUnixSocket() const
	{
		return DataRouterUnixSocket;
	}

	const FString& GetDiagnosticsFilename() const
	{
		return DiagnosticsFilename;
	}

	bool GetAllowToBeContacted() const
	{
		return bAllowToBeContacted;
	}

	bool GetHideAllowToBeContactedOption() const
	{
		return bHideAllowToBeContactedOption;
	}

	bool GetSendLogFile() const
	{
		return bSendLogFile;
	}

	bool GetHideLogFilesOption() const
	{
		return bHideLogFilesOption;
	}

	bool GetHideRestartOption() const
	{
		return bHideRestartOption;
	}

	bool IsAllowedToCloseWithoutSending() const
	{
		return bIsAllowedToCloseWithoutSending;
	}

	bool IsAllowedToSendWithoutDetailedInfo() const
	{
		return bIsAllowedToSendWithoutDetailedInfo;
	}

	bool GetShowEndpointInTooltip() const
	{
		return bShowEndpointInTooltip;
	}

	const FString& GetCompanyName() const
	{
		return CompanyName;
	}

#if PLATFORM_WINDOWS
	bool IsAllowedToCopyFilesToClipboard() const
	{
		return bIsAllowedToCopyFilesToClipboard;
	}
#endif

	int GetUserCommentSizeLimit() const
	{
		return UserCommentSizeLimit;
	}

	void SetAllowToBeContacted( bool bNewValue );
	void SetSendLogFile( bool bNewValue );


	void ApplyProjectOverrides(const FString& ConfigFilePath);

	/** Set config values that are determined by the crashing application saving a config file to the crash folder */
	void SetProjectConfigOverrides(const FConfigFile& InConfigFile);

	/**
	 * @return location for full crash dump for the specified branch.
	 */
	FString GetFullCrashDumpLocationForBranch(const FString& BranchName) const;

	/**
	 * Prints the current settings to the log. Call once all applicable overrides has been applied.
	 */
	void PrintSettingsToLog() const;

protected:
	/** Reads FFullCrashDumpEntry config entries. */
	void ReadFullCrashDumpConfigurations();

	/** Client version (two digit licensee built e.g. "1.0" - three digits for Epic builds e.g. "1.0.0") */
	FString CrashReportClientVersion;

	/** IP address of crash report receiver. */
	FString CrashReportReceiverIP;

	/** URL of Data Router service */
	FString DataRouterUrl;

	/** Local Unix Socket to use for Data Router service */
	FString DataRouterUnixSocket;

	/** Filename to use when saving diagnostics report, if generated locally. */
	FString DiagnosticsFilename;

	/** Section for crash report client configuration. */
	FString SectionName;

	/** Configuration used for copying full dump crashes. */
	TArray<FFullCrashDumpEntry> FullCrashDumpConfigurations;

	/**
	*	Whether the user allowed us to be contacted.
	*	If true the following properties are retrieved from the system: UserName (for non-launcher build) and EpicAccountID.
	*	Otherwise they will be empty.
	*/
	bool bAllowToBeContacted;

	/** Whether the user is shown the option to be contacted. */
	bool bHideAllowToBeContactedOption;

	/** Whether the user allowed us to send the log file. */
	bool bSendLogFile;

	/** Whether the user is shown the option to enable/disable sending the log file. */
	bool bHideLogFilesOption;

	/** Whether the user is shown the option to restart the crashed application */
	bool bHideRestartOption;

	/** Whether the user is allowed to close the crash reporter without sending a report */
	bool bIsAllowedToCloseWithoutSending;

	/** Whether the user is allowed to send the crash without providing detailed information about the crash */
	bool bIsAllowedToSendWithoutDetailedInfo;

	/** Optional string displayed in the "allow to be contacted" text. */
	FString CompanyName;

	/** Whether to show the endpoint (reciever or datarouter url) */
	bool bShowEndpointInTooltip;

#if PLATFORM_WINDOWS
	/** Whether the user is allowed to copy the report files to the clipboard */
	bool bIsAllowedToCopyFilesToClipboard;
#endif

	/** Size limit for the description of multi-line text */
	int UserCommentSizeLimit;
};
