// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleInterface.h"

DECLARE_LOG_CATEGORY_EXTERN(CrashReportCoreLog, Log, All)

class FCrashReportCoreModule : public IModuleInterface
{
public:
	virtual void StartupModule();
	virtual void ShutdownModule();

private:
};
