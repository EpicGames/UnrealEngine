// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CrashUpload.h"
#include "PlatformErrorReport.h"

/**
 * Implementation of the crash report client used for unattended uploads
 */
class FCrashReportCoreUnattended
{
public:
	/**
	 * Set up uploader object
	 * @param ErrorReport Error report to upload
	 */
	explicit FCrashReportCoreUnattended(FPlatformErrorReport& InErrorReport, bool InExitWhenComplete = true);

    bool IsUploadComplete() { return bUploadComplete; }
    
private:
	/**
	 * Update received every second
	 * @param DeltaTime Time since last update, unused
	 * @return Whether the updates should continue
	 */
	bool Tick(float DeltaTime);

	/**
	 * Begin calling Tick once a second
	 */
	void StartTicker();

	/** Object that uploads report files to the server */
	FCrashUploadToReceiver ReceiverUploader;

	/** Object that uploads report files to the server */
	FCrashUploadToDataRouter DataRouterUploader;

	/** Platform code for accessing the report */
	FPlatformErrorReport ErrorReport;
    
    /** upload is complete */
    bool bUploadComplete;
	
	/** exit when complete */
	bool bExitWhenComplete;
};
