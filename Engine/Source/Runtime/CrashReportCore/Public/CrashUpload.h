// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Interfaces/IHttpRequest.h"
#include "Internationalization/Text.h"
#include "PlatformErrorReport.h"
#include "ProfilingDebugging/ScopedTimers.h"
#include "Templates/SharedPointer.h"

struct FCompressedData;
struct FCompressedHeader;

class FCrashUploadBase
{
public:
	FCrashUploadBase();

	virtual ~FCrashUploadBase();

	/**
	 * Is this uploader enabled or disabled?
	 */
	bool IsEnabled() const { return State != EUploadState::Disabled; }

	/**
	 * Has BeginUpload been called?
	 */
	bool IsUploadCalled() const { return bUploadCalled; }

	/**
	 * Provide progress or error information for the UI
	 */
	const FText& GetStatusText() const { return UploadStateText; }

	/**
	 * Determine whether the upload has finished (successfully or otherwise)
	 * @return Whether the upload has finished
	 */
	bool IsFinished() const
	{
		return State >= EUploadState::FirstCompletedState;
	}

	/**
	 * Determine whether the upload has finished successfully.
	 */
	bool IsFinishedSuccessfully() const
	{
		return State == EUploadState::Finished;
	}

	void Cancel()
	{
		SetCurrentState(EUploadState::Cancelled);
	}

	static bool IsInitialized() { return bInitialized; }

	static void StaticInitialize(const FPlatformErrorReport& PlatformErrorReport);

	static void StaticShutdown();

protected:
	/** State enum to keep track of what the uploader is doing */
	struct EUploadState
	{
		enum Type
		{
			NotSet,

			PingingServer,
			Ready,
			CheckingReport,
			CheckingReportDetail,
			CompressAndSendData,
			WaitingToPostReportComplete,
			PostingReportComplete,
			Finished,

			ServerNotAvailable,
			UploadError,
			Cancelled,
			Disabled,

			FirstCompletedState = Finished,
		};
	};

	static bool CompressData(const TArray<FString>& InPendingFiles, struct FCompressedData& OutCompressedData, TArray<uint8>& OutPostData, struct FCompressedHeader* OptionalHeader = nullptr);

	/**
	* Get a string representation of the state, for logging purposes
	* @param State Value to stringize
	* @return Literal string value
	*/
	static const TCHAR* ToString(EUploadState::Type InState);

	/**
	* Set the current state, also updating the status text where necessary
	* @param State State the uploader is now in
	*/
	void SetCurrentState(EUploadState::Type InState);

	/**
	* When failed, add the report to a file list containing reports to upload next time
	*/
	void AddReportToFailedList() const;

	/**
	* Removes crash report folder once submitted to the crash reporting backend, as to not fill up server
	* -CleanCrashReports must be specified on the crash report client command line
	*/
	void CleanCrashReportDirectory(const FString& CrashReportDirectory);

protected:
	bool bUploadCalled;

	/** What this class is currently doing */
	EUploadState::Type State;

	/** Status of upload to display */
	FText UploadStateText;

	/** State to pause at until confirmation has been received to continue */
	EUploadState::Type PauseState;

	/** Full paths of files still to be uploaded */
	TArray<FString> PendingFiles;

	/** Error report being processed */
	FPlatformErrorReport ErrorReport;

	/** Buffer to keep reusing for file content and other messages */
	TArray<uint8> PostData;

	int32 PendingReportDirectoryIndex;

protected:
	static bool bInitialized;

	/** Full paths of reports from previous runs still to be uploaded */
	static TArray<FString> PendingReportDirectories;

	/** Full paths of reports from this run that did not upload */
	static TArray<FString> FailedReportDirectories;
};


/**
 * Handles uploading files to the crash report server
 */
class FCrashUploadToReceiver : public FCrashUploadBase
{
public:
	/**
	 * Constructor: pings server 
	 * @param ServerAddress Host IP of the crash report server
	 */
	explicit FCrashUploadToReceiver(const FString& InReceiverAddress);

	/**
	 * Destructor for logging
	 */
	virtual ~FCrashUploadToReceiver();

	/**
	 * Commence upload when ready
	 * @param PlatformErrorReport Error report to upload files from
	 */
	void BeginUpload(const FPlatformErrorReport& PlatformErrorReport);

private:
	/**
	 * Send a request to see if the server will accept this report
	 * @return Whether request was successfully sent
	 */
	bool SendCheckReportRequest();

	/**
	 * Compresses all crash report files and sends one compressed file.
	 */
	void CompressAndSendData();

	/**
	 * Convert the report name to single byte non-zero-terminated HTTP post data
	 */
	void AssignReportIdToPostDataBuffer();

	/**
	 * Send a POST request to the server indicating that all the files for the current report have been sent
	 */
	void PostReportComplete();

	/**
	 * Callback from HTTP library when a request has completed
	 * @param HttpRequest The request object
	 * @param HttpResponse The response from the server
	 * @param bSucceeded Whether a response was successfully received
	 */
	void OnProcessRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded);

	/**
	 * Start uploading if BeginUpload has been called
	 */
	void OnPingSuccess();

	/**
	 * Callback a set amount of time after ping request was sent
	 * @note Gets fired no matter whether ping response was received
	 * @param Unused time since last call
	 * @return Always returns false, meaning one-shot
	 */
	bool PingTimeout(float DeltaTime);

	/**
	 * If there a no pending files, look through pending reports for files to upload
	 */
	void CheckPendingReportsForFilesToUpload();

	/**
	 * Start uploading files, either when user presses Submit or Ping request succeeds, whichever is later
	 */
	void BeginUploadImpl();

	/**
	 * Create a request object and bind this class's response handler to it
	 */
	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> CreateHttpRequest();

	/**
	 * Send a ping request to the server
	 */
	void SendPingRequest();

	/**
	 * Parse an XML response from the server for the success field
	 * @param Response			Response to get message to parse from
	 * @param OutValidReport	Answer from the server on whether to continue with this report upload
	 * @return					Whether a valid response was received from the server
	 */
	static bool ParseServerResponse(FHttpResponsePtr Response, bool& OutValidReport);

	/** Host, port and common prefix of all requests to the server */
	FString UrlPrefix;
};

/**
* Handles uploading files to the data router
*/
class FCrashUploadToDataRouter : public FCrashUploadBase
{
public:
	/**
	 * Constructor: pings server
	 * @param ServerAddress Host IP of the crash report server
	 */
	explicit FCrashUploadToDataRouter(const FString& InDataRouterUrl, const FString& InOptionalDataRouterUnixSocket);

	/**
	 * Destructor for logging
	 */
	virtual ~FCrashUploadToDataRouter();

	void BeginUpload(const FPlatformErrorReport& PlatformErrorReport);

	/**
	 * Compresses all optional attachments files and sends one compressed file.
	 * Requires the base report to have been already sent (via BeginUpload).
	 */
	void BeginUploadOptionalAttachments(const FPlatformErrorReport& PlatformErrorReport);

	/**
	 * Compresses all crash report files and sends one compressed file.
	 */
	void CompressAndSendData();

	/**
	 * Create a request object and bind this class's response handler to it
	 */
	TSharedRef<IHttpRequest, ESPMode::ThreadSafe> CreateHttpRequest();

	/**
	 * Callback from HTTP library when a request has completed
	 * @param HttpRequest The request object
	 * @param HttpResponse The response from the server
	 * @param bSucceeded Whether a response was successfully received
	 */
	void OnProcessRequestComplete(FHttpRequestPtr HttpRequest, FHttpResponsePtr HttpResponse, bool bSucceeded);

private:
	/**
	 * If there a no pending files, look through pending reports for files to upload
	 */
	void CheckPendingReportsForFilesToUpload();

private:
	/** Url for data router requests */
	FString DataRouterUrl;

	/** Local Unix Socket for data router */
	FString DataRouterUnixSocket;

	/** HTTP request result */
	bool bResult = false;
	
	/** HTTP response code */
	int32 ResponseCode = 0;

	/** Payload size */
	uint32 PayloadSize = 0;

	/** Reports uploaded */
	uint32 ReportCount = 0;

	/** Duration of compression + upload */
	double Duration = 0.0;

	/** Timer */
	FDurationTimer Timer;
};
