// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CrashDescription.h"
#include "HAL/Platform.h"
#include "Misc/Paths.h"

class FCrashDebugHelperModule;
class FText;
struct FPrimaryCrashProperties;

/**
 * Helper that works with Windows Error Reports
 */
class FGenericErrorReport
{
public:
	/**
	 * Default constructor: creates a report with no files
	 */
	FGenericErrorReport()
	{
	}

	/**
	 * Discover all files in the crash report directory
	 * @param Directory Full path to directory containing the report
	 */
	CRASHREPORTCORE_API explicit FGenericErrorReport(const FString& Directory);

	/**
	 * One-time initialisation: does nothing by default
	 */
	static void Init()
	{
	}

	/**
	 * One-time clean-up: does nothing by default
	 */
	static void ShutDown()
	{
	}

	/**
	 * Write the provided comment into the error report
	 * @param UserComment Information provided by the user to add to the report
	 * @return Whether the comment was successfully written to the report
	 */
	bool SetUserComment(const FText& UserComment);

	/** Sets properties specific to the processed crash. Used to convert the old data into the crash context format. */
	void SetPrimaryCrashProperties( FPrimaryCrashProperties& out_PrimaryCrashProperties );

	/** Sets the version string in the error report and saves the change */
	void SetCrashReportClientVersion(const FString& InVersion);

	/**
	 * Provide full paths to all the report files
	 * @return List of paths
	 */
	CRASHREPORTCORE_API TArray<FString> GetFilesToUpload() const;

	/**
	 * Provide full paths to all the report's optional attachment files
	 * @return List of paths
	 */
	CRASHREPORTCORE_API TArray<FString> GetOptionalAttachmentsToUpload() const;

	/**
	 * Load the WER XML file for this report
	 * @note This is Windows specific and so shouldn't really be part of the public interface, but currently the server
	 * is Windows-specific in its checking of reports, so this is needed.
	 * @param OutString String to load the file into
	 * @return Whether finding and loading the file succeeded
	 */
	bool LoadWindowsReportXmlFile( FString& OutString ) const;

	/**
	 * @return Whether the file was found and successfully read
	 */
	bool TryReadDiagnosticsFile();

	/**
	 * Provide the full path of the error report directory
	 */
	FString GetReportDirectory() const
	{
		return ReportDirectory;
	}

	/**
	 * Provide the name of the error report directory
	 */
	FString GetReportDirectoryLeafName() const
	{
		// Using GetPathLeaf to actually get directory name
		return FPaths::GetPathLeaf(ReportDirectory);
	}

	/**
	 * Get the name of the crashed app from the report
	 */
	CRASHREPORTCORE_API FString FindCrashedAppName() const;

	/**
	 * Get the full path of the crashed app from the report
	 */
	CRASHREPORTCORE_API FString FindCrashedAppPath() const;

	/**
	 * Is there anything to upload?
	 */
	bool HasFilesToUpload() const
	{
		return ReportFilenames.Num() != 0;
	}


	/**
	 * Look thought the list of report files to find one with the given extension
	 * @return Whether a file with the extension was found
	 */
	CRASHREPORTCORE_API bool FindFirstReportFileWithExtension(FString& OutFilename, const TCHAR* Extension) const;

	/**
	 * Delete files from disk
	 */
	void DeleteFiles();
	
protected:
	/** Full path to the directory the report files are in */
	FString ReportDirectory;

	/** List of leaf filenames of all the files in the report folder */
	TArray<FString> ReportFilenames;

	/** Full path to the directory the report's optional attachment files are in */
	FString OptionalAttachmentsDirectory;

	/** List of leaf filenames of all the files in the report's optional attachments folder */
	TArray<FString> OptionalAttachmentsFilenames;

	/** Whether the error report generated an valid callstack. */
	mutable bool bValidCallstack;
};
