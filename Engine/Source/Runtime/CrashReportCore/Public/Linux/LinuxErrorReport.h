// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"
#include "GenericErrorReport.h"

struct FTimespan;

/**
 * Helper that works with Linux Error Reports
 */
class FLinuxErrorReport : public FGenericErrorReport
{
public:
	/**
	 * Default constructor: creates a report with no files
	 */
	FLinuxErrorReport()
	{
	}

	/**
	 * Discover all files in the crash report directory
	 * @param Directory Full path to directory containing the report
	 */
	explicit FLinuxErrorReport(const FString& Directory)
		: FGenericErrorReport(Directory)
	{
	}

	/**
	 * Do nothing - shouldn't be called on Linux
	 * @return Dummy text
	 */
	FText DiagnoseReport() const	
	{
		return FText::FromString("No local diagnosis on Linux");
	}

	/**
	 * Do nothing - shouldn't be called on Linux
	 * @return Empty string
	 */
	static void FindMostRecentErrorReports(TArray<FString>& ErrorReportPaths, const FTimespan& MaxCrashReportAge)
	{
		// The report folder is currently always sent on the command-line on Linux
	}

	/**
	 * Get the full path of the crashed app from the report
	 */
	FString FindCrashedAppPath() const
	{
		FString AppPath = FPaths::Combine(FPrimaryCrashProperties::Get()->BaseDir, FPrimaryCrashProperties::Get()->ExecutableName);
		return AppPath;
	}
};
