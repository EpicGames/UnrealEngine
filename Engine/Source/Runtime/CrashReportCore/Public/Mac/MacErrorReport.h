// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "../GenericErrorReport.h"

/**
 * Helper that works with Mac Error Reports
 */
class FMacErrorReport : public FGenericErrorReport
{
public:
	/**
	 * Default constructor: creates a report with no files
	 */
	FMacErrorReport()
	{
	}
	
	/**
	 * Load helper modules
	 */
	static void Init();
	
	/**
	 * Unload helper modules
	 */
	static void ShutDown();

	/**
	 * Discover all files in the crash report directory
	 * @param Directory Full path to directory containing the report
	 */
	CRASHREPORTCORE_API explicit FMacErrorReport(const FString& Directory);

	/**
	 * Parse the callstack from the Apple-style crash report log
	 * @return UE crash diagnosis text
	 */
	CRASHREPORTCORE_API FText DiagnoseReport() const;

	/**
	 * Get the full path of the crashed app from the report (hides implementation in FGenericErrorReport)
	 */
	CRASHREPORTCORE_API FString FindCrashedAppPath() const;

	/**
	 * Look for the most recent Mac Error Report
	 * @return Full path to the most recent report, or an empty string if none found
	 */
	static void FindMostRecentErrorReports(TArray<FString>& ErrorReportPaths, const FTimespan& MaxCrashReportAge);
};
