// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

#if PLATFORM_WINDOWS

#include "Windows/WindowsErrorReport.h"

typedef FWindowsErrorReport FPlatformErrorReport;

#elif PLATFORM_LINUX

#include "Linux/LinuxErrorReport.h"

typedef FLinuxErrorReport FPlatformErrorReport;

#elif PLATFORM_MAC

#include "Mac/MacErrorReport.h"

typedef FMacErrorReport FPlatformErrorReport;

#elif PLATFORM_IOS

#include "IOS/IOSErrorReport.h"

typedef FIOSErrorReport FPlatformErrorReport;

#elif PLATFORM_ANDROID

#include "Android/AndroidErrorReport.h"

typedef FAndroidErrorReport FPlatformErrorReport;

#else

typedef FGenericErrorReport FPlatformErrorReport;

#endif // PLATFORM_LINUX
