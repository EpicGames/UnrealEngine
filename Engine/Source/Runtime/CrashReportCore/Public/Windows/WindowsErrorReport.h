// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GenericErrorReport.h"

/**
 * Helper that works with Windows Error Reports
 */
class FWindowsErrorReport : public FGenericErrorReport
{
public:
	/**
	 * Default constructor: creates a report with no files
	 */
	FWindowsErrorReport()
	{
	}

	/**
	 * Load helper modules
	 */
	static void Init();

	/**
	 * Unload helper modules
	 */
	static void ShutDown();

	/**
	 * Discover all files in the crash report directory
	 * @param Directory Full path to directory containing the report
	 */
	CRASHREPORTCORE_API explicit FWindowsErrorReport(const FString& Directory);

	/**
	 * Provide the exception and a call-stack as plain text if possible
	 * @note This can take quite a long time
	 */
	CRASHREPORTCORE_API FText DiagnoseReport() const;

	/**
	 * Get the full path of the crashed app from the report
	 */
	CRASHREPORTCORE_API FString FindCrashedAppPath() const;

	/**
	 * Look for the most recent Windows Error Reports
	 * @return Full path to the most recent report, or an empty string if none found
	 */
	static void FindMostRecentErrorReports(TArray<FString>& ErrorReportPaths, const FTimespan& MaxCrashReportAge);
};
